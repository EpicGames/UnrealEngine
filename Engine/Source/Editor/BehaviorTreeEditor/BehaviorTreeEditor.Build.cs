// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class BehaviorTreeEditor : ModuleRules
{
	public BehaviorTreeEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AIGraph",
				"AIModule",
				"ApplicationCore",
				"AssetDefinition",
				"Core",
				"CoreUObject",
				"EditorWidgets",
				"Engine",
				"GraphEditor",
				"GameplayTags",
				"InputCore",
				"KismetWidgets",
				"PropertyEditor",
				"RewindDebuggerInterface",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"TraceServices",
				"UnrealEd",
			}
		);
	}
}