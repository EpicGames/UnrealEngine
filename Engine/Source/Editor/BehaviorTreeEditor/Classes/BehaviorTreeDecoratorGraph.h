// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BehaviorTreeDecoratorGraph.generated.h"

#define UE_API BEHAVIORTREEEDITOR_API

class UBehaviorTreeDecoratorGraphNode;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UBehaviorTreeDecoratorGraph : public UEdGraph
{
	GENERATED_UCLASS_BODY()

	UE_API void CollectDecoratorData(TArray<class UBTDecorator*>& DecoratorInstances, TArray<struct FBTDecoratorLogic>& DecoratorOperations) const;
	UE_API int32 SpawnMissingNodes(const TArray<class UBTDecorator*>& NodeInstances, const TArray<struct FBTDecoratorLogic>& Operations, int32 StartIndex);

protected:

	UE_API const class UBehaviorTreeDecoratorGraphNode* FindRootNode() const;
	UE_API void CollectDecoratorDataWorker(const class UBehaviorTreeDecoratorGraphNode* Node, TArray<class UBTDecorator*>& DecoratorInstances, TArray<struct FBTDecoratorLogic>& DecoratorOperations) const;

	UE_API UEdGraphPin* FindFreePin(UEdGraphNode* Node, EEdGraphPinDirection Direction);
	UE_API UBehaviorTreeDecoratorGraphNode* SpawnMissingNodeWorker(const TArray<class UBTDecorator*>& NodeInstances, const TArray<struct FBTDecoratorLogic>& Operations, int32& Index,
		UEdGraphNode* ParentGraphNode, int32 ChildIdx);

};

#undef UE_API
