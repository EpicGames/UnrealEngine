// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTree/BTCompositeNode.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BehaviorTreeDecoratorGraphNode.generated.h"

class UEdGraphPin;
class UEdGraphSchema;
class UObject;

UCLASS()
class UBehaviorTreeDecoratorGraphNode : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

	/** Get the BT graph that owns this node */
	virtual class UBehaviorTreeDecoratorGraph* GetDecoratorGraph();

	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;

	// @return the input pin for this state
	virtual UEdGraphPin* GetInputPin(int32 InputIndex=0) const;
	// @return the output pin for this state
	virtual UEdGraphPin* GetOutputPin(int32 InputIndex=0) const;

	virtual EBTDecoratorLogic::Type GetOperationType() const;

	/** allow creating / removing inputs */
	uint32 bAllowModifingInputs : 1;

	virtual void NodeConnectionListChanged() override;

	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const override;

	virtual bool CanUserDeleteNode() const override;
};
