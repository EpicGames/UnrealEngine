// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AIGraphTypes.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTreeDecoratorGraphNode.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "BehaviorTreeDecoratorGraphNode_Decorator.generated.h"

class UObject;
struct FGraphNodeClassData;

UCLASS()
class UBehaviorTreeDecoratorGraphNode_Decorator : public UBehaviorTreeDecoratorGraphNode 
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<UObject> NodeInstance;

	UPROPERTY()
	FGraphNodeClassData ClassData;

	virtual void PostPlacedNewNode() override;
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;

	virtual EBTDecoratorLogic::Type GetOperationType() const override;

	virtual void PostEditImport() override;
	virtual void PostEditUndo() override;
	virtual void PrepareForCopying() override;
	void PostCopyNode();
	bool RefreshNodeClass();
	void UpdateNodeClassData();

protected:
	friend class UBehaviorTreeGraphNode_CompositeDecorator;
	virtual void ResetNodeOwner();
};
