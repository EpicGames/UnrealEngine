// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTreeDecoratorGraphNode.h"
#include "Containers/EnumAsByte.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BehaviorTreeDecoratorGraphNode_Logic.generated.h"

class UEdGraphPin;
class UObject;

UENUM()
namespace EDecoratorLogicMode
{
	enum Type : int
	{
		Sink,
		And,
		Or,
		Not,
	};
}

UCLASS()
class UBehaviorTreeDecoratorGraphNode_Logic : public UBehaviorTreeDecoratorGraphNode
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TEnumAsByte<EDecoratorLogicMode::Type> LogicMode;

	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void AllocateDefaultPins() override;
	virtual bool CanUserDeleteNode() const override;
	virtual EBTDecoratorLogic::Type GetOperationType() const override;

	virtual void GetMenuEntries(struct FGraphContextMenuBuilder& ContextMenuBuilder) const override;

	bool CanRemovePins() const;
	bool CanAddPins() const;

	UEdGraphPin* AddInputPin();
	void RemoveInputPin(class UEdGraphPin* Pin);

	EDecoratorLogicMode::Type GetLogicMode(EBTDecoratorLogic::Type Op) const;
};
