// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Math/NumericLimits.h"
#include "Math/Vector2D.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Types/SlateVector2.h"

#include "BehaviorTreeEditorTypes.generated.h"

class FName;

struct FAbortDrawHelper
{
	uint16 AbortStart;
	uint16 AbortEnd;
	uint16 SearchStart;
	uint16 SearchEnd;

	FAbortDrawHelper() : AbortStart(MAX_uint16), AbortEnd(MAX_uint16), SearchStart(MAX_uint16), SearchEnd(MAX_uint16) {}
};

struct FCompareNodeXLocation
{
	FORCEINLINE bool operator()(const UEdGraphPin& A, const UEdGraphPin& B) const
	{
		const UEdGraphNode* NodeA = A.GetOwningNode();
		const UEdGraphNode* NodeB = B.GetOwningNode();

		if (NodeA->NodePosX == NodeB->NodePosX)
		{
			return NodeA->NodePosY < NodeB->NodePosY;
		}

		return NodeA->NodePosX < NodeB->NodePosX;
	}
};

namespace ESubNode
{
	enum Type {
		Decorator,
		Service
	};
}

struct FNodeBounds
{
	FDeprecateSlateVector2D Position;
	FDeprecateSlateVector2D Size;

	FNodeBounds(UE::Slate::FDeprecateVector2DParameter InPos, UE::Slate::FDeprecateVector2DParameter InSize)
		: Position(InPos)
		, Size(InSize)
	{
	}
};

UCLASS()
class UBehaviorTreeEditorTypes : public UObject
{
	GENERATED_UCLASS_BODY()

	static const FName PinCategory_MultipleNodes;
	static const FName PinCategory_SingleComposite;
	static const FName PinCategory_SingleTask;
	static const FName PinCategory_SingleNode;
};
