// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BehaviorTreeFactory.generated.h"

#define UE_API BEHAVIORTREEEDITOR_API

class FFeedbackContext;
class UClass;
class UObject;

UCLASS(MinimalAPI)
class UBehaviorTreeFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory interface
	UE_API virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	UE_API virtual bool CanCreateNew() const override;
	// End of UFactory interface
};



#undef UE_API
