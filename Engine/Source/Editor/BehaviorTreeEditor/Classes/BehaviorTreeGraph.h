// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AIGraph.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BehaviorTreeGraph.generated.h"

#define UE_API BEHAVIORTREEEDITOR_API

class UBehaviorTreeGraphNode_Root;
class UEdGraphNode;
class UObject;

UCLASS(MinimalAPI)
class UBehaviorTreeGraph : public UAIGraph
{
	GENERATED_UCLASS_BODY()

	enum EUpdateFlags
	{
		RebuildGraph = 0,
		ClearDebuggerFlags = 1,
		KeepRebuildCounter = 2,
	};

	/** increased with every graph rebuild, used to refresh data from subtrees */
	UPROPERTY()
	int32 ModCounter;

	UPROPERTY()
	bool bIsUsingModCounter;

	UPROPERTY()
	TSubclassOf<UBehaviorTreeGraphNode_Root> RootNodeClass;

	UE_API virtual void OnCreated() override;
	UE_API virtual void OnLoaded() override;
	UE_API virtual void Initialize() override;
	UE_API void OnSave();

	UE_API virtual void UpdateVersion() override;
	UE_API virtual void MarkVersion() override;
	UE_API virtual void UpdateAsset(int32 UpdateFlags = 0) override;
	UE_API virtual void OnSubNodeDropped() override;

	virtual bool DoesSupportServices() const { return true; }

	UE_API void UpdateBlackboardChange();
	UE_API void UpdateAbortHighlight(struct FAbortDrawHelper& Mode0, struct FAbortDrawHelper& Mode1);
	UE_API void CreateBTFromGraph(class UBehaviorTreeGraphNode* RootEdNode);
	UE_API void SpawnMissingNodes();
	UE_API void UpdatePinConnectionTypes();
	UE_API bool UpdateInjectedNodes();
	UE_API void UpdateBrokenComposites();
	UE_API class UEdGraphNode* FindInjectedNode(int32 Index);
	UE_API void ReplaceNodeConnections(UEdGraphNode* OldNode, UEdGraphNode* NewNode);
	UE_API void RebuildExecutionOrder();
	UE_API void RebuildChildOrder(UEdGraphNode* ParentNode);
	UE_API void SpawnMissingNodesForParallel();
	UE_API void RemoveUnknownSubNodes();

	UE_API void AutoArrange();

protected:

	UE_API void CollectAllNodeInstances(TSet<UObject*>& NodeInstances) override;

#if WITH_EDITOR
	UE_API virtual void PostEditUndo() override;
#endif

	UE_API void UpdateVersion_UnifiedSubNodes();
	UE_API void UpdateVersion_InnerGraphWhitespace();
	UE_API void UpdateVersion_RunBehaviorInSeparateGraph();
};

#undef UE_API
