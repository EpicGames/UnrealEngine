// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AIGraphNode.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BehaviorTreeGraphNode.generated.h"

#define UE_API BEHAVIORTREEEDITOR_API

class ISlateStyle;
class UBehaviorTreeGraph;
class UEdGraph;
class UEdGraphSchema;
class UObject;
template <typename T> struct TObjectPtr;

UCLASS(MinimalAPI)
class UBehaviorTreeGraphNode : public UAIGraphNode
{
	GENERATED_UCLASS_BODY()

	/** only some of behavior tree nodes support decorators */
	UPROPERTY()
	TArray<TObjectPtr<UBehaviorTreeGraphNode>> Decorators;

	/** only some of behavior tree nodes support services */
	UPROPERTY()
	TArray<TObjectPtr<UBehaviorTreeGraphNode>> Services;

	//~ Begin UEdGraphNode Interface
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const override;
	UE_API virtual void FindDiffs(class UEdGraphNode* OtherNode, struct FDiffResults& Results) override;
	UE_API virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	//~ End UEdGraphNode Interface

#if WITH_EDITOR
	UE_API virtual void PostEditUndo() override;
#endif

	UE_API virtual FText GetDescription() const override;
	UE_API virtual bool HasErrors() const override;
	UE_API virtual void InitializeInstance() override;
	UE_API virtual void OnSubNodeAdded(UAIGraphNode* SubNode) override;
	UE_API virtual void OnSubNodeRemoved(UAIGraphNode* SubNode) override;
	UE_API virtual void RemoveAllSubNodes() override;
	UE_API virtual int32 FindSubNodeDropIndex(UAIGraphNode* SubNode) const override;
	UE_API virtual void InsertSubNodeAt(UAIGraphNode* SubNode, int32 DropIndex) override;
	UE_API virtual void UpdateErrorMessage() override;

	UE_DEPRECATED(5.4, "Use GetOwnerBehaviorTreeGraph instead.")
	UE_API virtual UBehaviorTreeGraph* GetBehaviorTreeGraph();
	UE_API virtual UBehaviorTreeGraph* GetOwnerBehaviorTreeGraph() const;

	UE_API virtual FLinearColor GetBackgroundColor(bool bIsActiveForDebugger) const;

	/** check if node can accept breakpoints */
	virtual bool CanPlaceBreakpoints() const { return false; }

	UE_API void ClearDebuggerState();

	/** gets the style set from which GetNameIcon is queried */
	UE_API virtual const ISlateStyle& GetNameIconStyleSet() const;

	/** gets icon resource name for title bar */
	UE_API virtual FName GetNameIcon() const;

	/** if set, this node was injected from subtree and shouldn't be edited */
	UPROPERTY()
	uint32 bInjectedNode : 1;

	/** if set, this node is root of tree or sub node of it */
	uint32 bRootLevel : 1;

	/** if set, observer setting is invalid (injected nodes only) */
	uint32 bHasObserverError : 1;

	/** highlighting nodes in abort range for more clarity when setting up decorators */
	uint32 bHighlightInAbortRange0 : 1;

	/** highlighting nodes in abort range for more clarity when setting up decorators */
	uint32 bHighlightInAbortRange1 : 1;

	/** highlighting connections in search range for more clarity when setting up decorators */
	uint32 bHighlightInSearchRange0 : 1;

	/** highlighting connections in search range for more clarity when setting up decorators */
	uint32 bHighlightInSearchRange1 : 1;

	/** highlighting nodes during quick find */
	uint32 bHighlightInSearchTree : 1;

	/** highlight other child node indexes when hovering over a child */
	uint32 bHighlightChildNodeIndices : 1;

	/** debugger flag: breakpoint exists */
	uint32 bHasBreakpoint : 1;

	/** debugger flag: breakpoint is enabled */
	uint32 bIsBreakpointEnabled : 1;

	/** debugger flag: mark node as active (current state) */
	uint32 bDebuggerMarkCurrentlyActive : 1;

	/** debugger flag: mark node as active (browsing previous states) */
	uint32 bDebuggerMarkPreviouslyActive : 1;

	/** debugger flag: briefly flash active node */
	uint32 bDebuggerMarkFlashActive : 1;

	/** debugger flag: mark as succeeded search path */
	uint32 bDebuggerMarkSearchSucceeded : 1;

	/** debugger flag: mark as failed on search path */
	uint32 bDebuggerMarkSearchFailed : 1;

	/** debugger flag: mark as trigger of search path */
	uint32 bDebuggerMarkSearchTrigger : 1;

	/** debugger flag: mark as trigger of discarded search path */
	uint32 bDebuggerMarkSearchFailedTrigger : 1;

	/** debugger flag: mark as going to parent */
	uint32 bDebuggerMarkSearchReverseConnection : 1;

	/** debugger flag: mark stopped on this breakpoint */
	uint32 bDebuggerMarkBreakpointTrigger : 1;

	/** debugger variable: index on search path */
	int32 DebuggerSearchPathIndex;

	/** debugger variable: number of nodes on search path */
	int32 DebuggerSearchPathSize;

	/** debugger variable: incremented on change of debugger flags for render updates */
	int32 DebuggerUpdateCounter;

	/** used to show node's runtime description rather than static one */
	FString DebuggerRuntimeDescription;

protected:

	/** creates add decorator... submenu */
	UE_API void CreateAddDecoratorSubMenu(class UToolMenu* Menu, UEdGraph* Graph) const;

	/** creates add service... submenu */
	UE_API void CreateAddServiceSubMenu(class UToolMenu* Menu, UEdGraph* Graph) const;

	/** add right click menu to create subnodes: Decorators */
	UE_API void AddContextMenuActionsDecorators(class UToolMenu* Menu, const FName SectionName, class UGraphNodeContextMenuContext* Context) const;

	/** add right click menu to create subnodes: Services */
	UE_API void AddContextMenuActionsServices(class UToolMenu* Menu, const FName SectionName, class UGraphNodeContextMenuContext* Context) const;
};

#undef UE_API
