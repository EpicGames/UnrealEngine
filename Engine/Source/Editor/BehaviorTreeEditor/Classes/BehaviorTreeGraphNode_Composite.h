// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTreeGraphNode.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BehaviorTreeGraphNode_Composite.generated.h"

#define UE_API BEHAVIORTREEEDITOR_API

class UObject;

UCLASS(MinimalAPI)
class UBehaviorTreeGraphNode_Composite : public UBehaviorTreeGraphNode
{
	GENERATED_UCLASS_BODY()
	
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetDescription() const override;
	UE_API virtual FText GetTooltipText() const override;
	virtual bool RefreshNodeClass() override{ return false; }

	/** Gets a list of actions that can be done to this particular node */
	UE_API virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;

	/** check if node can accept breakpoints */
	virtual bool CanPlaceBreakpoints() const override { return true; }

	UE_API virtual FLinearColor GetBackgroundColor(bool bIsActiveForDebugger) const override;

protected:

	UE_API virtual void PostPasteNode() override;
};

#undef UE_API
