// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTreeGraphNode.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "BehaviorTreeGraphNode_CompositeDecorator.generated.h"

#define UE_API BEHAVIORTREEEDITOR_API

class UBehaviorTreeDecoratorGraph;
class UEdGraph;
class UObject;

UCLASS(MinimalAPI)
class UBehaviorTreeGraphNode_CompositeDecorator : public UBehaviorTreeGraphNode
{
	GENERATED_UCLASS_BODY()

	// The logic graph for this decorator (returning a boolean)
	UPROPERTY()
	TObjectPtr<class UEdGraph> BoundGraph;

	UPROPERTY(EditAnywhere, Category=Description)
	FString CompositeName;

	/** if set, all logic operations will be shown in description */
	UPROPERTY(EditAnywhere, Category=Description)
	uint32 bShowOperations : 1;

	/** updated with internal graph changes, set when decorators inside can abort flow */
	UPROPERTY()
	uint32 bCanAbortFlow : 1;

	uint32 bHasBrokenInstances : 1;

	TSubclassOf<UBehaviorTreeDecoratorGraph> GraphClass;

	UE_API FString GetNodeTypeDescription() const;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FText GetDescription() const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual void PostPlacedNewNode() override;
	UE_API virtual void PostLoad() override;
	virtual UEdGraph* GetBoundGraph() const override { return BoundGraph; }
	UE_API virtual bool IsSubNode() const override;
	UE_API virtual bool HasErrors() const override;
	UE_API virtual bool RefreshNodeClass() override;
	UE_API virtual void UpdateNodeClassData() override;

	UE_API virtual void PrepareForCopying() override;
	UE_API virtual void PostCopyNode() override;
	UE_API virtual void PostPasteNode() override;

	UE_API int32 SpawnMissingNodes(const TArray<class UBTDecorator*>& NodeInstances, const TArray<struct FBTDecoratorLogic>& Operations, int32 StartIndex);
	UE_API void CollectDecoratorData(TArray<class UBTDecorator*>& NodeInstances, TArray<struct FBTDecoratorLogic>& Operations) const;
	UE_API void SetDecoratorData(class UBTCompositeNode* InParentNode, uint8 InChildIndex);
	UE_API void InitializeDecorator(class UBTDecorator* InnerDecorator);
	UE_API void OnBlackboardUpdate();
	UE_API void OnInnerGraphChanged();
	UE_API void BuildDescription();
	UE_API void UpdateBrokenInstances();

	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	UE_API virtual FLinearColor GetBackgroundColor(bool bIsActiveForDebugger) const override;

	UE_API void ResetExecutionRange();

	/** Execution index range of internal nodes, used by debugger */
	uint16 FirstExecutionIndex;
	uint16 LastExecutionIndex;

protected:
	UE_API void CreateBoundGraph();
	UE_API virtual void ResetNodeOwner() override;

	UPROPERTY()
	TObjectPtr<class UBTCompositeNode> ParentNodeInstance;

	uint8 ChildIndex;

	UPROPERTY()
	FString CachedDescription;
};

#undef UE_API
