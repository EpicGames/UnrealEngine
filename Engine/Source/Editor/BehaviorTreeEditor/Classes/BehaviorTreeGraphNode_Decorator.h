// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTreeGraphNode.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BehaviorTreeGraphNode_Decorator.generated.h"

class UObject;

UCLASS(MinimalAPI)
class UBehaviorTreeGraphNode_Decorator : public UBehaviorTreeGraphNode
{
	GENERATED_UCLASS_BODY()

	BEHAVIORTREEEDITOR_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	BEHAVIORTREEEDITOR_API virtual void AllocateDefaultPins() override;
	BEHAVIORTREEEDITOR_API virtual FLinearColor GetBackgroundColor(bool bIsActiveForDebugger) const override;
	BEHAVIORTREEEDITOR_API void CollectDecoratorData(TArray<class UBTDecorator*>& NodeInstances, TArray<struct FBTDecoratorLogic>& Operations) const;
};
