// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTreeGraphNode.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "BehaviorTreeGraphNode_Root.generated.h"

#define UE_API BEHAVIORTREEEDITOR_API

class UObject;

/** Root node of this behavior tree, holds Blackboard data */
UCLASS(MinimalAPI)
class UBehaviorTreeGraphNode_Root : public UBehaviorTreeGraphNode
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category="AI|BehaviorTree")
	TObjectPtr<class UBlackboardData> BlackboardAsset;

	UE_API virtual void PostPlacedNewNode() override;
	UE_API virtual void AllocateDefaultPins() override;
	virtual bool CanDuplicateNode() const override { return false; }
	virtual bool CanUserDeleteNode() const override{ return false; }
	virtual bool HasErrors() const override { return false; }
	virtual bool RefreshNodeClass() override { return false; }
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	/** gets icon resource name for title bar */
	UE_API virtual FName GetNameIcon() const override;
	UE_API virtual FText GetTooltipText() const override;

	UE_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual FText GetDescription() const override;

	UE_API virtual FLinearColor GetBackgroundColor(bool bIsActiveForDebugger) const override;

	/** notify behavior tree about blackboard change */
	UE_API void UpdateBlackboard();
};

#undef UE_API
