// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTreeGraphNode.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BehaviorTreeGraphNode_Service.generated.h"

class UObject;

UCLASS(MinimalAPI)
class UBehaviorTreeGraphNode_Service : public UBehaviorTreeGraphNode
{
	GENERATED_UCLASS_BODY()

	BEHAVIORTREEEDITOR_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	BEHAVIORTREEEDITOR_API virtual void AllocateDefaultPins() override;
	BEHAVIORTREEEDITOR_API virtual FLinearColor GetBackgroundColor(bool bIsActiveForDebugger) const override;
};

