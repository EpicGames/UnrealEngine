// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTreeGraphNode_Composite.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BehaviorTreeGraphNode_SimpleParallel.generated.h"

class FString;
class UEdGraphPin;
class UObject;

UCLASS()
class UBehaviorTreeGraphNode_SimpleParallel : public UBehaviorTreeGraphNode_Composite
{
	GENERATED_UCLASS_BODY()

	virtual void AllocateDefaultPins() override;
	virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
};
