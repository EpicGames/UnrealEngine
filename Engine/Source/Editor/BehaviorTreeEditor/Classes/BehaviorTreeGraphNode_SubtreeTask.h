// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTreeGraphNode_Task.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BehaviorTreeGraphNode_SubtreeTask.generated.h"

class UObject;

UCLASS(MinimalAPI)
class UBehaviorTreeGraphNode_SubtreeTask : public UBehaviorTreeGraphNode_Task
{
	GENERATED_UCLASS_BODY()

	/** UBehaviorTreeGraph.UpdateCounter value of subtree graph */
	int32 SubtreeVersion;

	/** path of behavior tree asset used to create injected nodes preview */
	FString SubtreePath;

	/** updates nodes injected from subtree's root */
	bool UpdateInjectedNodes();

	virtual FLinearColor GetBackgroundColor(bool bIsActiveForDebugger) const override;
};
