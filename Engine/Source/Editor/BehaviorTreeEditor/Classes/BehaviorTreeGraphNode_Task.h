// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTreeGraphNode.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BehaviorTreeGraphNode_Task.generated.h"

class UObject;

UCLASS(MinimalAPI)
class UBehaviorTreeGraphNode_Task : public UBehaviorTreeGraphNode
{
	GENERATED_UCLASS_BODY()

	BEHAVIORTREEEDITOR_API virtual void AllocateDefaultPins() override;
	BEHAVIORTREEEDITOR_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	/** Gets a list of actions that can be done to this particular node */
	BEHAVIORTREEEDITOR_API virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;

	virtual bool CanPlaceBreakpoints() const override { return true; }
	BEHAVIORTREEEDITOR_API virtual FLinearColor GetBackgroundColor(bool bIsActiveForDebugger) const override;
};
