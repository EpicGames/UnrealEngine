// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AIGraphSchema.h"
#include "AIGraphTypes.h"
#include "BehaviorTreeGraphNode_CompositeDecorator.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "EdGraphSchema_BehaviorTree.generated.h"

#define UE_API BEHAVIORTREEEDITOR_API

class FSlateRect;
class UBehaviorTreeGraphNode_CompositeDecorator;
class UBehaviorTreeGraphNode_Decorator;
class UBehaviorTreeGraphNode_Service;
class UBehaviorTreeGraphNode_Task;
class UClass;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UObject;
struct FEdGraphPinType;
struct FGraphNodeClassData;

/** Action to auto arrange the graph */
USTRUCT()
struct FBehaviorTreeSchemaAction_AutoArrange : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	FBehaviorTreeSchemaAction_AutoArrange() 
		: FEdGraphSchemaAction() {}

	FBehaviorTreeSchemaAction_AutoArrange(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

UCLASS(MinimalAPI)
class UEdGraphSchema_BehaviorTree : public UAIGraphSchema
{
	GENERATED_UCLASS_BODY()

	//~ Begin EdGraphSchema Interface
	UE_API virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const override;
	UE_API virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	UE_API virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	UE_API virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	UE_API virtual const FPinConnectionResponse CanMergeNodes(const UEdGraphNode* A, const UEdGraphNode* B) const override;
	UE_API virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	UE_API virtual class FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
	UE_API virtual bool IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const override;
	UE_API virtual int32 GetCurrentVisualizationCacheID() const override;
	UE_API virtual void ForceVisualizationCacheClear() const override;
	//~ End EdGraphSchema Interface

	UE_API virtual void GetGraphNodeContextActions(FGraphContextMenuBuilder& ContextMenuBuilder, int32 SubNodeFlags) const override;
	UE_API virtual void GetSubNodeClasses(int32 SubNodeFlags, TArray<FGraphNodeClassData>& ClassData, UClass*& GraphNodeClass) const override;

	TSubclassOf<UBehaviorTreeGraphNode_CompositeDecorator> CompositeDecoratorClass;
	TSubclassOf<UBehaviorTreeGraphNode_Decorator> DecoratorClass;
	TSubclassOf<UBehaviorTreeGraphNode_Task> TaskClass;
	TSubclassOf<UBehaviorTreeGraphNode_Service> ServiceClass;

protected:
	UE_API virtual FGraphNodeClassHelper& GetClassCache() const;
	UE_API virtual bool IsNodeSubtreeTask(const FGraphNodeClassData& NodeClass) const;

private:
	// ID for checking dirty status of node titles against, increases whenever 
	static UE_API int32 CurrentCacheRefreshID;
};

#undef UE_API
