// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#include "EdGraphSchema_BehaviorTreeDecorator.generated.h"

#define UE_API BEHAVIORTREEEDITOR_API

class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UObject;
struct FEdGraphPinType;
struct FGraphNodeClassHelper;

/** Action to add a node to the graph */
USTRUCT()
struct FDecoratorSchemaAction_NewNode : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	/** Template of node we want to create */
	UPROPERTY()
	TObjectPtr<class UBehaviorTreeDecoratorGraphNode> NodeTemplate;

	FDecoratorSchemaAction_NewNode() 
		: FEdGraphSchemaAction()
		, NodeTemplate(nullptr)
	{}

	FDecoratorSchemaAction_NewNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
		, NodeTemplate(nullptr)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	//~ End FEdGraphSchemaAction Interface

	template <typename NodeType>
	static NodeType* SpawnNodeFromTemplate(class UEdGraph* ParentGraph, NodeType* InTemplateNode, const FVector2f& Location)
	{
		FDecoratorSchemaAction_NewNode Action;
		Action.NodeTemplate = InTemplateNode;

		return Cast<NodeType>(Action.PerformAction(ParentGraph, NULL, Location));
	}
};

UCLASS(MinimalAPI)
class UEdGraphSchema_BehaviorTreeDecorator : public UEdGraphSchema
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FString PC_Boolean;

	UE_API void AddPin(class UEdGraphNode* InGraphNode);
	UE_API void RemovePin(class UEdGraphPin* InGraphPin);

	//~ Begin EdGraphSchema Interface
	UE_API virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const override;
	UE_API virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	UE_API virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual FName GetParentContextMenuName() const override { return NAME_None; }
	UE_API virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	UE_API virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	UE_API virtual bool ShouldHidePinDefaultValue(UEdGraphPin* Pin) const override;
	UE_API virtual bool IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const override;
	UE_API virtual int32 GetCurrentVisualizationCacheID() const override;
	UE_API virtual void ForceVisualizationCacheClear() const override;
	//~ End EdGraphSchema Interface

	static UE_API TSharedPtr<FDecoratorSchemaAction_NewNode> AddNewDecoratorAction(FGraphContextMenuBuilder& ContextMenuBuilder, const FText& Category, const FText& MenuDesc, const FText& Tooltip);

protected:
	UE_API virtual FGraphNodeClassHelper& GetClassCache() const;

private:
	inline static int32 CurrentCacheRefreshID = 0;
};

#undef UE_API
