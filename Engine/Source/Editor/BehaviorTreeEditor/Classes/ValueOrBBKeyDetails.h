// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API BEHAVIORTREEEDITOR_API

namespace ESelectInfo
{
	enum Type : int;
}

class IPropertyHandle;
class IPropertyUtilities;
class SWidget;
class UBlackboardData;
struct FValueOrBlackboardKeyBase;

class FValueOrBBKeyDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();

	UE_API virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	UE_API virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	UE_API bool CanEditDefaultValue() const;

protected:
	TSharedPtr<IPropertyHandle> StructProperty;
	TSharedPtr<IPropertyHandle> KeyProperty;
	TSharedPtr<IPropertyHandle> DefaultValueProperty;
	TSharedPtr<IPropertyUtilities> CachedUtils;
	TArray<FName> MatchingKeys;

	UE_API virtual void ValidateData();
	UE_API virtual TSharedRef<SWidget> CreateDefaultValueWidget();

	UE_API void GetMatchingKeys(TArray<FName>& OutNames);
	UE_API bool HasAccessToBlackboard() const;
	UE_API TSharedRef<SWidget> OnGetKeyNames();
	UE_API void OnKeyChanged(int32 Index);
	UE_API FText GetKeyDesc() const;
	UE_API const FValueOrBlackboardKeyBase* GetDataPtr() const;
};

class FValueOrBBKeyDetails_Class : public FValueOrBBKeyDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:
	TSharedPtr<IPropertyHandle> BaseClassProperty;

	virtual void ValidateData() override;
	virtual TSharedRef<SWidget> CreateDefaultValueWidget() override;

	void OnBaseClassChanged();
	void OnSetClass(const UClass* NewClass);
	const UClass* OnGetSelectedClass() const;
	void BrowseToClass() const;
};

class FValueOrBBKeyDetails_Enum : public FValueOrBBKeyDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:
	TSharedPtr<IPropertyHandle> EnumTypeProperty;
	TSharedPtr<IPropertyHandle> NativeEnumTypeNameProperty;

	virtual void ValidateData() override;
	virtual TSharedRef<SWidget> CreateDefaultValueWidget() override;

	void OnEnumSelectionChanged(int32 NewValue, ESelectInfo::Type);
	void OnEnumTypeChanged();
	void OnNativeEnumTypeNameChanged();
	int32 GetEnumValue() const;
	bool CanEditEnumType() const;
};

class FValueOrBBKeyDetails_Object : public FValueOrBBKeyDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:
	TSharedPtr<IPropertyHandle> BaseClassProperty;

	virtual void ValidateData() override;
	virtual TSharedRef<SWidget> CreateDefaultValueWidget() override;

	void OnBaseClassChanged();
	void OnObjectChanged(const FAssetData& AssetData);
	FString OnGetObjectPath() const;
	void BrowseToObject() const;
};

class FValueOrBBKeyDetails_Struct : public FValueOrBBKeyDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:
	TSharedPtr<IPropertyHandle> EditDefaultsOnlyProperty;
};

class FValueOrBBKeyDetails_WithChild : public FValueOrBBKeyDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();
	UE_API virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
};

#undef UE_API
