// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_BehaviorTree.h"
#include "BehaviorTreeEditor.h"
#include "BehaviorTreeEditorModule.h"
#include "BehaviorTree/BlackboardData.h"
#include "BehaviorTree/BehaviorTree.h"
#include "SBehaviorTreeDiff.h"
#include "Framework/Application/SlateApplication.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_BehaviorTree)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FText UAssetDefinition_BehaviorTree::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_BehaviorTree", "Behavior Tree");
}

FLinearColor UAssetDefinition_BehaviorTree::GetAssetColor() const
{
	return FColor(149,70,255);
}

TSoftClassPtr<> UAssetDefinition_BehaviorTree::GetAssetClass() const
{
	return UBehaviorTree::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_BehaviorTree::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::AI, LOCTEXT("AssetDefinition_BehaviorTreeSubMenu", "NPC"), ECategoryMenuType::Section) };
	return Categories;
}

EAssetCommandResult UAssetDefinition_BehaviorTree::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const EToolkitMode::Type Mode = OpenArgs.GetToolkitMode();

	for (UBehaviorTree* BehaviorTree : OpenArgs.LoadObjects<UBehaviorTree>())
	{
		// check if we have an editor open for this BT's blackboard & use that if we can
		bool bFoundExisting = false;
		if (BehaviorTree->BlackboardAsset != nullptr)
		{
			constexpr bool bFocusIfOpen = false;
			FBehaviorTreeEditor* ExistingInstance = static_cast<FBehaviorTreeEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(BehaviorTree->BlackboardAsset, bFocusIfOpen));
			if (ExistingInstance != nullptr && ExistingInstance->GetBehaviorTree() == nullptr)
			{
				ExistingInstance->InitBehaviorTreeEditor(Mode, OpenArgs.ToolkitHost, BehaviorTree);
				bFoundExisting = true;
			}
		}

		if (!bFoundExisting)
		{
			FBehaviorTreeEditorModule& BehaviorTreeEditorModule = FModuleManager::GetModuleChecked<FBehaviorTreeEditorModule>("BehaviorTreeEditor");
			BehaviorTreeEditorModule.CreateBehaviorTreeEditor(Mode, OpenArgs.ToolkitHost, BehaviorTree);
		}
	}
	return EAssetCommandResult::Handled;
}

EAssetCommandResult UAssetDefinition_BehaviorTree::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	UBehaviorTree* OldBehaviorTree = Cast<UBehaviorTree>(DiffArgs.OldAsset);
	UBehaviorTree* NewBehaviorTree = Cast<UBehaviorTree>(DiffArgs.NewAsset);
	check(NewBehaviorTree != nullptr || OldBehaviorTree != nullptr);

	// sometimes we're comparing different revisions of one single asset (other 
	// times we're comparing two completely separate assets altogether)
	const bool bIsSingleAsset = !NewBehaviorTree || !OldBehaviorTree || (NewBehaviorTree->GetName() == OldBehaviorTree->GetName());

	FText WindowTitle = LOCTEXT("NamelessBehaviorTreeDiff", "Behavior Tree Diff");
	// if we're diffing one asset against itself 
	if (bIsSingleAsset)
	{
		// identify the assumed single asset in the window's title
		FString TreeName;
		if (NewBehaviorTree)
		{
			TreeName = NewBehaviorTree->GetName();
		}
		else if (OldBehaviorTree)
		{
			TreeName = OldBehaviorTree->GetName();
		}
		WindowTitle = FText::Format(LOCTEXT("Behavior Tree Diff", "{0} - Behavior Tree Diff"), FText::FromString(TreeName));
	}

	const TSharedPtr<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.ClientSize(FVector2f(1000.f,800.f));

	Window->SetContent
	(SNew(SBehaviorTreeDiff)
		.BehaviorTreeOld(OldBehaviorTree)
		.BehaviorTreeNew(NewBehaviorTree)
		.OldRevision(DiffArgs.OldRevision)
		.NewRevision(DiffArgs.NewRevision)
		.ShowAssetNames(!bIsSingleAsset)
		.OpenInDefaults_UObject(this, &UAssetDefinition_BehaviorTree::OpenInDefaults));

	// Make this window a child of the modal window if we've been spawned while one is active.
	const TSharedPtr<SWindow> ActiveModal = FSlateApplication::Get().GetActiveModalWindow();
	if (ActiveModal.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(Window.ToSharedRef(), ActiveModal.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window.ToSharedRef());
	}
	return EAssetCommandResult::Handled;
}

void UAssetDefinition_BehaviorTree::OpenInDefaults(class UBehaviorTree* OldBehaviorTree, class UBehaviorTree* NewBehaviorTree) const
{
	FAssetDiffArgs DiffArgs;
	DiffArgs.OldAsset = OldBehaviorTree;
	DiffArgs.NewAsset = NewBehaviorTree;
	Super::PerformAssetDiff(DiffArgs);
}

#undef LOCTEXT_NAMESPACE
