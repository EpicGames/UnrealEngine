// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_Blackboard.h"
#include "BehaviorTreeEditorModule.h"
#include "BehaviorTree/BlackboardData.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_Blackboard)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FText UAssetDefinition_Blackboard::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_Blackboard", "Blackboard");
}

FLinearColor UAssetDefinition_Blackboard::GetAssetColor() const
{
	return FColor(201, 29, 85);
}

TSoftClassPtr<> UAssetDefinition_Blackboard::GetAssetClass() const
{
	return UBlackboardData::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_Blackboard::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::AI, LOCTEXT("AssetDefinition_BlackboardSubMenu", "NPC"), ECategoryMenuType::Section) };
	return Categories;
}

EAssetCommandResult UAssetDefinition_Blackboard::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	FBehaviorTreeEditorModule& BehaviorTreeEditorModule = FModuleManager::LoadModuleChecked<FBehaviorTreeEditorModule>("BehaviorTreeEditor");
	for (UBlackboardData* BlackboardData : OpenArgs.LoadObjects<UBlackboardData>())
	{
		BehaviorTreeEditorModule.CreateBehaviorTreeEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, BlackboardData);
	}
	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
