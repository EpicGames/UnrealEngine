// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeConnectionDrawingPolicy.h"

#include "BehaviorTreeColors.h"
#include "BehaviorTreeDebugger.h"
#include "BehaviorTreeGraphNode.h"
#include "ConnectionDrawingPolicy.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/PlatformMath.h"
#include "Math/Color.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"

class FSlateRect;

FBehaviorTreeConnectionDrawingPolicy::FBehaviorTreeConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj)
	: FAIGraphConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements, InGraphObj)
{
}

void FBehaviorTreeConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params)
{
	Params.AssociatedPin1 = OutputPin;
	Params.AssociatedPin2 = InputPin;
	Params.WireThickness = 1.5f;

	Params.WireColor = BehaviorTreeColors::Connection::Default;

	const bool bDeemphasizeUnhoveredPins = HoveredPins.Num() > 0;
	if (bDeemphasizeUnhoveredPins)
	{
		ApplyHoverDeemphasis(OutputPin, InputPin, /*inout*/ Params.WireThickness, /*inout*/ Params.WireColor);
	}

	UBehaviorTreeGraphNode* FromNode = OutputPin ? Cast<UBehaviorTreeGraphNode>(OutputPin->GetOwningNode()) : NULL;
	UBehaviorTreeGraphNode* ToNode = InputPin ? Cast<UBehaviorTreeGraphNode>(InputPin->GetOwningNode()) : NULL;
	if (ToNode && FromNode)
	{
		if ((ToNode->bDebuggerMarkCurrentlyActive && FromNode->bDebuggerMarkCurrentlyActive) ||
			(ToNode->bDebuggerMarkPreviouslyActive && FromNode->bDebuggerMarkPreviouslyActive))
		{
			Params.WireThickness = 10.0f;
			Params.bDrawBubbles = true;
		}
		else if (FBehaviorTreeDebugger::IsPlaySessionPaused())
		{
			UBehaviorTreeGraphNode* FirstToNode = ToNode;
			int32 FirstPathIdx = ToNode->DebuggerSearchPathIndex;
			for (int32 i = 0; i < ToNode->Decorators.Num(); i++)
			{
				UBehaviorTreeGraphNode* TestNode = ToNode->Decorators[i];
				if (TestNode->DebuggerSearchPathIndex != INDEX_NONE &&
					(TestNode->bDebuggerMarkSearchSucceeded || TestNode->bDebuggerMarkSearchFailed))
				{
					if (TestNode->DebuggerSearchPathIndex < FirstPathIdx || FirstPathIdx == INDEX_NONE)
					{
						FirstPathIdx = TestNode->DebuggerSearchPathIndex;
						FirstToNode = TestNode;
					}
				}
			}

			if (FirstToNode->bDebuggerMarkSearchSucceeded || FirstToNode->bDebuggerMarkSearchFailed)
			{
				Params.WireThickness = 5.0f;
				Params.WireColor = FirstToNode->bDebuggerMarkSearchSucceeded ? BehaviorTreeColors::Debugger::SearchSucceeded :
				BehaviorTreeColors::Debugger::SearchFailed;

				// Use the bUserFlag1 flag to indicate that we need to reverse the direction of connection (used by debugger)
				Params.bUserFlag1 = true;
			}
		}
	}
}
