// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AIGraphConnectionDrawingPolicy.h"
#include "HAL/Platform.h"

#define UE_API BEHAVIORTREEEDITOR_API

class FSlateRect;
class FSlateWindowElementList;
class UEdGraph;
class UEdGraphPin;
struct FConnectionParams;

// This class draws the connections for an UEdGraph with a behavior tree schema
class FBehaviorTreeConnectionDrawingPolicy : public FAIGraphConnectionDrawingPolicy
{
public:
	//
	UE_API FBehaviorTreeConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj);

	// FConnectionDrawingPolicy interface 
	UE_API virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;
	// End of FConnectionDrawingPolicy interface
};

#undef UE_API
