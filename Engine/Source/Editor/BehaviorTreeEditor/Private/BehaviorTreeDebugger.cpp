// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeDebugger.h"
#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTree/BehaviorTreeManager.h"
#include "BehaviorTree/BTAuxiliaryNode.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/Tasks/BTTask_RunBehaviorDynamic.h"
#include "BehaviorTreeDelegates.h"
#include "BehaviorTreeEditor.h"
#include "BehaviorTreeGraphNode_CompositeDecorator.h"
#include "BehaviorTreeGraphNode_Decorator.h"
#include "BehaviorTreeGraphNode_Root.h"
#include "BehaviorTreeGraphNode_Service.h"
#include "EdGraph/EdGraph.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Selection.h"
#include "EngineGlobals.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Controller.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "IRewindDebugger.h"
#include "UnrealEdGlobals.h"

FBehaviorTreeDebugger::FBehaviorTreeDebugger()
{
	TreeAsset = NULL;
	bIsPIEActive = false;
	bIsCurrentSubtree = false;
	StoppedOnBreakpointExecutionIndex = INDEX_NONE;
	StepForwardIntoIdx = INDEX_NONE;
	StepForwardOverIdx = INDEX_NONE;
	StepBackIntoIdx = INDEX_NONE;
	StepBackOverIdx = INDEX_NONE;
	StepOutIdx = INDEX_NONE;
	SavedTimestamp = 0.;
	CurrentTimestamp = 0.;
	RewindDebuggerTimestamp = 0.;

	FEditorDelegates::BeginPIE.AddRaw(this, &FBehaviorTreeDebugger::OnBeginPIE);
	FEditorDelegates::EndPIE.AddRaw(this, &FBehaviorTreeDebugger::OnEndPIE);
	FEditorDelegates::PausePIE.AddRaw(this, &FBehaviorTreeDebugger::OnPausePIE);

#if USE_BEHAVIORTREE_DEBUGGER
	UBehaviorTreeComponent::ActiveDebuggerCounter++;
#endif
}

FBehaviorTreeDebugger::~FBehaviorTreeDebugger()
{
	FEditorDelegates::BeginPIE.RemoveAll(this);
	FEditorDelegates::EndPIE.RemoveAll(this);
	FEditorDelegates::PausePIE.RemoveAll(this);
	USelection::SelectObjectEvent.RemoveAll(this);
	FBehaviorTreeDelegates::OnTreeStarted.RemoveAll(this);
	FBehaviorTreeDelegates::OnDebugLocked.RemoveAll(this);
	FBehaviorTreeDelegates::OnDebugSelected.RemoveAll(this);

#if USE_BEHAVIORTREE_DEBUGGER
	UBehaviorTreeComponent::ActiveDebuggerCounter--;
#endif
}

void FBehaviorTreeDebugger::CacheRootNode()
{
	if(RootNode.IsValid() || TreeAsset == nullptr || TreeAsset->BTGraph == nullptr)
	{
		return;
	}

	for (const auto& Node : TreeAsset->BTGraph->Nodes)
	{
		RootNode = Cast<UBehaviorTreeGraphNode_Root>(Node);
		if (RootNode.IsValid())
		{
			break;
		}
	}
}

void FBehaviorTreeDebugger::Setup(UBehaviorTree* InTreeAsset, TSharedRef<FBehaviorTreeEditor> InEditorOwner)
{
	EditorOwner = InEditorOwner;
	TreeAsset = InTreeAsset;
	DebuggerInstanceIndex = INDEX_NONE;
	ActiveStepIndex = 0;
	LastValidExecutionStepId = FBehaviorTreeExecutionStep::InvalidExecutionId;
	ActiveBreakpoints.Reset();
	KnownInstances.Reset();

	CacheRootNode();

#if USE_BEHAVIORTREE_DEBUGGER
	if (IsPIESimulating())
	{
		OnBeginPIE(GEditor->bIsSimulatingInEditor);

		Refresh();
	}
#endif
}

void FBehaviorTreeDebugger::Refresh()
{
	CacheRootNode();

	if (IsPIESimulating() && IsDebuggerReady())
	{
		// make sure is grabs data if currently paused
		if (IsPlaySessionPaused() && TreeInstance.IsValid())
		{
			FindLockedDebugActor(GEditor->PlayWorld);
			
			UpdateDebuggerInstance();
			UpdateAvailableActions();

			if (DebuggerInstanceIndex != INDEX_NONE)
			{
				UpdateDebuggerViewOnStepChange();
				UpdateDebuggerViewOnTick();

				FBehaviorTreeExecutionStep& ExecutionStep = TreeInstance->DebuggerSteps[ActiveStepIndex];
				const FBehaviorTreeDebuggerInstance& ShowInstance = ExecutionStep.InstanceStack[DebuggerInstanceIndex];
				OnActiveNodeChanged(ShowInstance.ActivePath, HasContinuousPrevStep() ?
					TreeInstance->DebuggerSteps[ActiveStepIndex - 1].InstanceStack[DebuggerInstanceIndex].ActivePath :
					TArray<uint16>());

				UpdateAssetFlags(ShowInstance, RootNode.Get(), ExecutionStep.ExecutionStepId);
			}
		}
	}
}

void FBehaviorTreeDebugger::Tick(float DeltaTime)
{
	if (TreeAsset == NULL)
	{
		return;
	}

	if ((!TreeInstance.IsValid() || !RootNode.IsValid())
		&& !IsPlaySessionPaused())
	{
		// clear state when active tree is lost
		if (DebuggerInstanceIndex != INDEX_NONE)
		{
			ClearDebuggerState();
		}

		return;
	}

#if USE_BEHAVIORTREE_DEBUGGER
	if (IsPlaySessionPaused())
	{
		if (const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance())
		{
			if (RewindDebugger->IsRecording() && TreeInstance.IsValid())
			{
				const double WorldTime = TreeInstance->GetWorld()->GetTimeSeconds();
				const double RewindDebuggerWorldTime = WorldTime - RewindDebugger->GetRecordingDuration() + RewindDebugger->GetScrubTime();
				if(RewindDebuggerWorldTime != RewindDebuggerTimestamp)
				{
					RewindDebuggerTimestamp = RewindDebuggerWorldTime;
					const int32 DebugStep = FindDebuggerStepForTimeStamp(RewindDebuggerTimestamp);
					if(DebugStep != INDEX_NONE && DebugStep != ActiveStepIndex)
					{
						const int32 PreviousStep = ActiveStepIndex != INDEX_NONE ? ActiveStepIndex : FMath::Max(DebugStep - 1, 0); // If debug step is 0 it means the previous step got lost to the max number of step recorded and initialize from itself which isn't ideal but best we can do.
						UpdateCurrentStep(PreviousStep, DebugStep, false); // Don't open asset while scrubbing to avoid stealing focus and opening tree that might not be of interest.
					}
				}
			}
		}
		return;
	}

	TArray<uint16> EmptyPath;

	int32 TestStepIndex = 0;

	if (LastValidExecutionStepId != FBehaviorTreeExecutionStep::InvalidExecutionId)
	{
		for (int32 Idx = TreeInstance->DebuggerSteps.Num() - 1; Idx >= 0; Idx--)
		{
			const FBehaviorTreeExecutionStep& Step = TreeInstance->DebuggerSteps[Idx];
			if (Step.ExecutionStepId == LastValidExecutionStepId)
			{
				TestStepIndex = Idx;
				break;
			}
		}
	}

	// Keep track if an update of the debugger instance is required after processing changes from displayed step to latest step
	// since it is possible the current instance gets invalidated (e.g. subtree) and that resets the current index. In such case
	// we need to wait for the next tick to recompute ActiveStepIndex and LastValidExecutionStepId.
	bool bDebuggerInstanceUpdateRequired = true;

	// find index of previously displayed state and notify about all changes in between to give breakpoints a chance to trigger
	for (int32 i = TestStepIndex; i < TreeInstance->DebuggerSteps.Num(); i++)
	{
		const FBehaviorTreeExecutionStep& Step = TreeInstance->DebuggerSteps[i];

		if (Step.ExecutionStepId > DisplayedExecutionStepId)
		{
			// If we're in an inactive step, and this tree does not appear in the current step, then we skip updating
			// the active step index. Without this, UpdateDebuggerInstance would call ClearDebuggerState (also resetting
			// ActiveStepIndex) as DebuggerInstanceIndex became invalid, but in the next frame Clear would not be
			// called, as DebuggerInstanceIndex hasn't changed (so ActiveStepIndex would be unchanged).
			// This inconsistency caused a flash of the state label, and then switched to SIMULATION in the next frame,
			// even though the subtree was not reachable anymore.
			if (ActiveStepIndex == INDEX_NONE)
			{
				const bool bAppearsInStep = Step.InstanceStack.ContainsByPredicate([this](
					const FBehaviorTreeDebuggerInstance& Instance)
				{
					return Instance.TreeAsset == TreeAsset;
				});

				if (!bAppearsInStep)
				{
					continue;
				}
			}

			ActiveStepIndex = i;
			LastValidExecutionStepId = Step.ExecutionStepId;

			UpdateDebuggerInstance();
			UpdateAvailableActions();

			bDebuggerInstanceUpdateRequired = false;

			if (DebuggerInstanceIndex != INDEX_NONE)
			{
				UpdateDebuggerViewOnStepChange();

				const FBehaviorTreeDebuggerInstance& ShowInstance = TreeInstance->DebuggerSteps[ActiveStepIndex].InstanceStack[DebuggerInstanceIndex];
				OnActiveNodeChanged(ShowInstance.ActivePath, HasContinuousPrevStep() ?
					TreeInstance->DebuggerSteps[ActiveStepIndex - 1].InstanceStack[DebuggerInstanceIndex].ActivePath :
					EmptyPath);
			}
		}

		// skip rest of them if breakpoint hit
		if (IsPlaySessionPaused())
		{
			break;
		}
	}

	if (bDebuggerInstanceUpdateRequired)
	{
		UpdateDebuggerInstance();
	}

	if (DebuggerInstanceIndex != INDEX_NONE)
	{
		FBehaviorTreeExecutionStep& ExecutionStep = TreeInstance->DebuggerSteps[ActiveStepIndex];
		const FBehaviorTreeDebuggerInstance& ShowInstance = ExecutionStep.InstanceStack[DebuggerInstanceIndex];

		if (UBTNode* InstanceRootNode = ShowInstance.RootNode.Get(); IsValidChecked(InstanceRootNode))
		{
			if (DisplayedExecutionStepId != ExecutionStep.ExecutionStepId)
			{
				UpdateAssetFlags(ShowInstance, RootNode.Get(), ExecutionStep.ExecutionStepId);
			}

			// Collect current runtime descriptions for every node when displaying the most recent step.
			// Otherwise we keep the descriptions from the displayed debugger step that were updated by UpdateAssetFlags.
			int32 LatestStepIndex = TreeInstance->DebuggerSteps.Num() - 1;
			if (ActiveStepIndex == LatestStepIndex)
			{
				TArray<FString> RuntimeDescriptions;
				TreeInstance->StoreDebuggerRuntimeValues(RuntimeDescriptions, InstanceRootNode, IntCastChecked<uint16>(DebuggerInstanceIndex));

				UpdateAssetRuntimeDescription(RuntimeDescriptions, RootNode.Get());
			}
		}
	}

	UpdateDebuggerViewOnTick();
#endif
}

bool FBehaviorTreeDebugger::IsTickable() const
{
	return IsDebuggerReady();
}

void FBehaviorTreeDebugger::OnBeginPIE(const bool bIsSimulating)
{
	bIsPIEActive = true;
	if (TSharedPtr<FBehaviorTreeEditor> Owner = EditorOwner.Pin())
	{
		Owner->RegenerateMenusAndToolbars();
		Owner->DebuggerUpdateGraph();
	}

	ActiveBreakpoints.Reset();
	CollectBreakpointsFromAsset(RootNode.Get());

	FindMatchingTreeInstance();

	// remove these delegates first as we can get multiple calls to OnBeginPIE()
	USelection::SelectObjectEvent.RemoveAll(this);
	FBehaviorTreeDelegates::OnTreeStarted.RemoveAll(this);
	FBehaviorTreeDelegates::OnDebugSelected.RemoveAll(this);

	USelection::SelectObjectEvent.AddRaw(this, &FBehaviorTreeDebugger::OnObjectSelected);
	FBehaviorTreeDelegates::OnTreeStarted.AddRaw(this, &FBehaviorTreeDebugger::OnTreeStarted);
	FBehaviorTreeDelegates::OnDebugSelected.AddRaw(this, &FBehaviorTreeDebugger::OnAIDebugSelected);
}

void FBehaviorTreeDebugger::OnEndPIE(const bool bIsSimulating)
{
	bIsPIEActive = false;
	if(TSharedPtr<FBehaviorTreeEditor> Owner = EditorOwner.Pin())
	{
		Owner->RegenerateMenusAndToolbars();
	}

	USelection::SelectObjectEvent.RemoveAll(this);
	FBehaviorTreeDelegates::OnTreeStarted.RemoveAll(this);
	FBehaviorTreeDelegates::OnDebugSelected.RemoveAll(this);

	ClearDebuggerState();
	ActiveBreakpoints.Reset();

	FBehaviorTreeDebuggerInstance EmptyData;
	UpdateAssetFlags(EmptyData, RootNode.Get(), FBehaviorTreeExecutionStep::InvalidExecutionId);
	UpdateDebuggerViewOnInstanceChange();
}

void FBehaviorTreeDebugger::OnPausePIE(const bool bIsSimulating)
{
#if USE_BEHAVIORTREE_DEBUGGER
	// We might have paused while executing a sub-tree, so make sure that the editor is showing the correct tree
	TSharedPtr<FBehaviorTreeEditor> EditorOwnerPin = EditorOwner.Pin();
	if (EditorOwnerPin.IsValid() && TreeInstance.IsValid() && TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex))
	{
		const FBehaviorTreeExecutionStep& StepInfo = TreeInstance->DebuggerSteps[ActiveStepIndex];
		const int32 LastInstanceIndex = StepInfo.InstanceStack.Num() - 1;
		if (StepInfo.InstanceStack.IsValidIndex(LastInstanceIndex) && StepInfo.InstanceStack[LastInstanceIndex].TreeAsset != TreeAsset)
		{
			EditorOwnerPin->DebuggerSwitchAsset(StepInfo.InstanceStack[LastInstanceIndex].TreeAsset);
		}
	}
#endif
}

void FBehaviorTreeDebugger::OnObjectSelected(UObject* Object)
{
	if (Object && Object->IsSelected())
	{
		UBehaviorTreeComponent* InstanceComp = FindInstanceInActor(Cast<AActor>(Object));
		if (InstanceComp)
		{
			ClearDebuggerState();
			TreeInstance = InstanceComp;

			UpdateDebuggerViewOnInstanceChange();
		}
	}	
}

void FBehaviorTreeDebugger::OnAIDebugSelected(const APawn* Pawn)
{
	UBehaviorTreeComponent* TestComp = FindInstanceInActor((APawn*)Pawn);
	if (TestComp)
	{
		ClearDebuggerState();
		TreeInstance = TestComp;

		UpdateDebuggerViewOnInstanceChange();
	}
}

void FBehaviorTreeDebugger::OnTreeStarted(const UBehaviorTreeComponent& OwnerComp, const UBehaviorTree& InTreeAsset)
{
	// start debugging if tree asset matches, and no other actor was selected
	if (!TreeInstance.IsValid() && TreeAsset && TreeAsset == &InTreeAsset)
	{
		ClearDebuggerState();
		TreeInstance = MakeWeakObjectPtr(const_cast<UBehaviorTreeComponent*>(&OwnerComp));

		UpdateDebuggerViewOnInstanceChange();
	}

	// update known instances
	TWeakObjectPtr<UBehaviorTreeComponent> KnownComp = const_cast<UBehaviorTreeComponent*>(&OwnerComp);
	KnownInstances.AddUnique(KnownComp);
}

void FBehaviorTreeDebugger::ClearDebuggerState(bool bKeepSubtree)
{
	LastValidExecutionStepId = bKeepSubtree ? LastValidExecutionStepId : FBehaviorTreeExecutionStep::InvalidExecutionId;
	DisplayedExecutionStepId = FBehaviorTreeExecutionStep::InvalidExecutionId;

	DebuggerInstanceIndex = INDEX_NONE;
	ActiveStepIndex = INDEX_NONE;

	if (TreeAsset && RootNode.IsValid())
	{
		FBehaviorTreeDebuggerInstance EmptyData;
		UpdateAssetFlags(EmptyData, RootNode.Get(), FBehaviorTreeExecutionStep::InvalidExecutionId);
	}
}

void FBehaviorTreeDebugger::UpdateDebuggerInstance()
{
	int32 PrevStackIndex = DebuggerInstanceIndex;
	DebuggerInstanceIndex = INDEX_NONE;

	if (TreeInstance.IsValid())
	{
#if USE_BEHAVIORTREE_DEBUGGER
		if (TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex))
		{
			const FBehaviorTreeExecutionStep& StepInfo = TreeInstance->DebuggerSteps[ActiveStepIndex];
			for (int32 i = 0; i < StepInfo.InstanceStack.Num(); i++)
			{
				if (StepInfo.InstanceStack[i].TreeAsset == TreeAsset)
				{
					DebuggerInstanceIndex = i;
					break;
				}
			}
		}
#endif
		UpdateCurrentSubtree();
	}

	if (DebuggerInstanceIndex != PrevStackIndex)
	{
		UpdateDebuggerViewOnInstanceChange();
	}
}

void FBehaviorTreeDebugger::SetNodeFlags(const struct FBehaviorTreeDebuggerInstance& Data, class UBehaviorTreeGraphNode* Node, class UBTNode* NodeInstance)
{
	const bool bIsNodeActivePath = Data.ActivePath.Contains(NodeInstance->GetExecutionIndex());
	const bool bIsNodeActiveAdditional = Data.AdditionalActiveNodes.Contains(NodeInstance->GetExecutionIndex());
	const bool bIsNodeActive = bIsNodeActivePath || bIsNodeActiveAdditional;
	const bool bIsShowingCurrentState = IsShowingCurrentState();

	Node->DebuggerUpdateCounter = DisplayedExecutionStepId;
	Node->bDebuggerMarkCurrentlyActive = bIsNodeActive && bIsShowingCurrentState;
	Node->bDebuggerMarkPreviouslyActive = bIsNodeActive && !bIsShowingCurrentState;
	
	const bool bIsTaskNode = NodeInstance->IsA(UBTTaskNode::StaticClass());
	Node->bDebuggerMarkFlashActive = bIsNodeActivePath && bIsTaskNode && IsPlaySessionRunning();
	Node->bDebuggerMarkSearchTrigger = false;
	Node->bDebuggerMarkSearchFailedTrigger = false;

	Node->bDebuggerMarkBreakpointTrigger = NodeInstance->GetExecutionIndex() == StoppedOnBreakpointExecutionIndex;
	if (Node->bDebuggerMarkBreakpointTrigger)
	{
		if (TSharedPtr<FBehaviorTreeEditor> Owner = EditorOwner.Pin())
		{
			Owner->JumpToNode(Node);
		}
	}

	int32 SearchPathIdx = INDEX_NONE;
	int32 NumTriggers = 0;
	bool bTriggerOnly = false;

	for (int32 i = 0; i < Data.PathFromPrevious.Num(); i++)
	{
		const FBehaviorTreeDebuggerInstance::FNodeFlowData& SearchStep = Data.PathFromPrevious[i];
		const bool bMatchesNodeIndex = (SearchStep.ExecutionIndex == NodeInstance->GetExecutionIndex());
		if (SearchStep.bTrigger || SearchStep.bDiscardedTrigger)
		{
			NumTriggers++;
			if (bMatchesNodeIndex)
			{
				Node->bDebuggerMarkSearchTrigger = SearchStep.bTrigger;
				Node->bDebuggerMarkSearchFailedTrigger = SearchStep.bDiscardedTrigger;
				bTriggerOnly = true;
			}
		}
		else if (bMatchesNodeIndex)
		{
			SearchPathIdx = i;
			bTriggerOnly = false;
		}
	}

	Node->bDebuggerMarkSearchSucceeded = (SearchPathIdx != INDEX_NONE) && Data.PathFromPrevious[SearchPathIdx].bPassed;
	Node->bDebuggerMarkSearchFailed = (SearchPathIdx != INDEX_NONE) && !Data.PathFromPrevious[SearchPathIdx].bPassed;
	Node->DebuggerSearchPathIndex = bTriggerOnly ? 0 : FMath::Max(-1, SearchPathIdx - NumTriggers);
	Node->DebuggerSearchPathSize = Data.PathFromPrevious.Num() - NumTriggers;
}

void FBehaviorTreeDebugger::SetCompositeDecoratorFlags(const struct FBehaviorTreeDebuggerInstance& Data, class UBehaviorTreeGraphNode_CompositeDecorator* Node)
{
	const bool bIsShowingCurrentState = IsShowingCurrentState();
	bool bIsNodeActive = false;
	for (int32 i = 0; i < Data.AdditionalActiveNodes.Num(); i++)
	{
		if (Node->FirstExecutionIndex <= Data.AdditionalActiveNodes[i] && Node->LastExecutionIndex >= Data.AdditionalActiveNodes[i])
		{
			bIsNodeActive = true;
			break;
		}
	}	
	
	Node->DebuggerUpdateCounter = DisplayedExecutionStepId;
	Node->bDebuggerMarkCurrentlyActive = bIsNodeActive && bIsShowingCurrentState;
	Node->bDebuggerMarkPreviouslyActive = bIsNodeActive && !bIsShowingCurrentState;

	Node->bDebuggerMarkFlashActive = false;
	Node->bDebuggerMarkSearchTrigger = false;
	Node->bDebuggerMarkSearchFailedTrigger = false;

	int32 SearchPathIdx = INDEX_NONE;
	int32 NumTriggers = 0;
	bool bTriggerOnly = false;
	for (int32 i = 0; i < Data.PathFromPrevious.Num(); i++)
	{
		const FBehaviorTreeDebuggerInstance::FNodeFlowData& SearchStep = Data.PathFromPrevious[i];
		const bool bMatchesNodeIndex = (Node->FirstExecutionIndex <= SearchStep.ExecutionIndex && Node->LastExecutionIndex >= SearchStep.ExecutionIndex);
		if (SearchStep.bTrigger || SearchStep.bDiscardedTrigger)
		{
			NumTriggers++;
			if (bMatchesNodeIndex)
			{
				Node->bDebuggerMarkSearchTrigger = SearchStep.bTrigger;
				Node->bDebuggerMarkSearchFailedTrigger = SearchStep.bDiscardedTrigger;
				bTriggerOnly = true;
			}
		}
		else if (bMatchesNodeIndex)
		{
			SearchPathIdx = i;
			bTriggerOnly = false;
		}
	}

	Node->bDebuggerMarkSearchSucceeded = (SearchPathIdx != INDEX_NONE) && Data.PathFromPrevious[SearchPathIdx].bPassed;
	Node->bDebuggerMarkSearchFailed = (SearchPathIdx != INDEX_NONE) && !Data.PathFromPrevious[SearchPathIdx].bPassed;
	Node->DebuggerSearchPathIndex = bTriggerOnly ? 0 : FMath::Max(-1, SearchPathIdx - NumTriggers);
	Node->DebuggerSearchPathSize = Data.PathFromPrevious.Num() - NumTriggers;
}

void FBehaviorTreeDebugger::UpdateAssetFlags(const struct FBehaviorTreeDebuggerInstance& Data, class UBehaviorTreeGraphNode* Node, int32 ExecutionStepId)
{
	if (Node == NULL)
	{
		return;
	}

	// special case for marking root when out of nodes
	if (Node == RootNode.Get())
	{
		const bool bIsNodeActive = (Data.ActivePath.Num() == 0) && (ExecutionStepId != FBehaviorTreeExecutionStep::InvalidExecutionId);
		const bool bIsShowingCurrentState = IsShowingCurrentState();

		Node->bDebuggerMarkCurrentlyActive = bIsNodeActive && bIsShowingCurrentState;
		Node->bDebuggerMarkPreviouslyActive = bIsNodeActive && !bIsShowingCurrentState;
		DisplayedExecutionStepId = ExecutionStepId;
	}

	for (int32 PinIdx = 0; PinIdx < Node->Pins.Num(); PinIdx++)
	{
		UEdGraphPin* Pin = Node->Pins[PinIdx];
		if (Pin->Direction != EGPD_Output)
		{
			continue;
		}

		for (int32 i = 0; i < Pin->LinkedTo.Num(); i++)
		{
			UBehaviorTreeGraphNode* LinkedNode = Cast<UBehaviorTreeGraphNode>(Pin->LinkedTo[i]->GetOwningNode());
			if (LinkedNode)
			{
				UBTNode* BTNode = Cast<UBTNode>(LinkedNode->NodeInstance);
				if (BTNode)
				{
					SetNodeFlags(Data, LinkedNode, BTNode);
					SetNodeRuntimeDescription(Data.RuntimeDesc, LinkedNode, BTNode);
				}

				for (int32 iAux = 0; iAux < LinkedNode->Decorators.Num(); iAux++)
				{
					UBehaviorTreeGraphNode_Decorator* DecoratorNode = Cast<UBehaviorTreeGraphNode_Decorator>(LinkedNode->Decorators[iAux]);
					UBTAuxiliaryNode* AuxNode = DecoratorNode ? Cast<UBTAuxiliaryNode>(DecoratorNode->NodeInstance) : NULL;
					if (AuxNode)
					{
						SetNodeFlags(Data, DecoratorNode, AuxNode);
						SetNodeRuntimeDescription(Data.RuntimeDesc, DecoratorNode, AuxNode);

						// pass restart trigger to parent graph node for drawing
						LinkedNode->bDebuggerMarkSearchTrigger |= DecoratorNode->bDebuggerMarkSearchTrigger;
						LinkedNode->bDebuggerMarkSearchFailedTrigger |= DecoratorNode->bDebuggerMarkSearchFailedTrigger;
					}

					UBehaviorTreeGraphNode_CompositeDecorator* CompDecoratorNode = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(LinkedNode->Decorators[iAux]);
					if (CompDecoratorNode)
					{
						SetCompositeDecoratorFlags(Data, CompDecoratorNode);
						SetCompositeDecoratorRuntimeDescription(Data.RuntimeDesc, CompDecoratorNode);

						// pass restart trigger to parent graph node for drawing
						LinkedNode->bDebuggerMarkSearchTrigger |= CompDecoratorNode->bDebuggerMarkSearchTrigger;
						LinkedNode->bDebuggerMarkSearchFailedTrigger |= CompDecoratorNode->bDebuggerMarkSearchFailedTrigger;
					}
				}

				for (int32 iAux = 0; iAux < LinkedNode->Services.Num(); iAux++)
				{
					UBehaviorTreeGraphNode_Service* ServiceNode = Cast<UBehaviorTreeGraphNode_Service>(LinkedNode->Services[iAux]);
					UBTAuxiliaryNode* AuxNode = ServiceNode ? Cast<UBTAuxiliaryNode>(ServiceNode->NodeInstance) : NULL;
					if (AuxNode)
					{
						SetNodeFlags(Data, ServiceNode, AuxNode);
						SetNodeRuntimeDescription(Data.RuntimeDesc, ServiceNode, AuxNode);
					}
				}

				UpdateAssetFlags(Data, LinkedNode, ExecutionStepId);
			}
		}
	}
}

void FBehaviorTreeDebugger::UpdateAssetRuntimeDescription(const TArray<FString>& RuntimeDescriptions, class UBehaviorTreeGraphNode* Node)
{
	if (Node == NULL)
	{
		return;
	}

	for (int32 PinIdx = 0; PinIdx < Node->Pins.Num(); PinIdx++)
	{
		UEdGraphPin* Pin = Node->Pins[PinIdx];
		if (Pin->Direction != EGPD_Output)
		{
			continue;
		}

		for (int32 i = 0; i < Pin->LinkedTo.Num(); i++)
		{
			UBehaviorTreeGraphNode* LinkedNode = Cast<UBehaviorTreeGraphNode>(Pin->LinkedTo[i]->GetOwningNode());
			if (LinkedNode)
			{
				UBTNode* BTNode = Cast<UBTNode>(LinkedNode->NodeInstance);
				if (BTNode)
				{
					SetNodeRuntimeDescription(RuntimeDescriptions, LinkedNode, BTNode);
				}

				for (int32 iAux = 0; iAux < LinkedNode->Decorators.Num(); iAux++)
				{
					UBehaviorTreeGraphNode_Decorator* DecoratorNode = Cast<UBehaviorTreeGraphNode_Decorator>(LinkedNode->Decorators[iAux]);
					UBTAuxiliaryNode* AuxNode = DecoratorNode ? Cast<UBTAuxiliaryNode>(DecoratorNode->NodeInstance) : NULL;
					if (AuxNode)
					{
						SetNodeRuntimeDescription(RuntimeDescriptions, DecoratorNode, AuxNode);
					}

					UBehaviorTreeGraphNode_CompositeDecorator* CompDecoratorNode = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(LinkedNode->Decorators[iAux]);
					if (CompDecoratorNode)
					{
						SetCompositeDecoratorRuntimeDescription(RuntimeDescriptions, CompDecoratorNode);
					}
				}

				for (int32 iAux = 0; iAux < LinkedNode->Services.Num(); iAux++)
				{
					UBehaviorTreeGraphNode_Service* ServiceNode = Cast<UBehaviorTreeGraphNode_Service>(LinkedNode->Services[iAux]);
					UBTAuxiliaryNode* AuxNode = ServiceNode ? Cast<UBTAuxiliaryNode>(ServiceNode->NodeInstance) : NULL;
					if (AuxNode)
					{
						SetNodeRuntimeDescription(RuntimeDescriptions, ServiceNode, AuxNode);
					}
				}

				UpdateAssetRuntimeDescription(RuntimeDescriptions, LinkedNode);
			}
		}
	}
}

void FBehaviorTreeDebugger::SetNodeRuntimeDescription(const TArray<FString>& RuntimeDescriptions, class UBehaviorTreeGraphNode* Node, class UBTNode* NodeInstance)
{
	Node->DebuggerRuntimeDescription = RuntimeDescriptions.IsValidIndex(NodeInstance->GetExecutionIndex()) ?
		RuntimeDescriptions[NodeInstance->GetExecutionIndex()] : FString();
}

void FBehaviorTreeDebugger::SetCompositeDecoratorRuntimeDescription(const TArray<FString>& RuntimeDescriptions, class UBehaviorTreeGraphNode_CompositeDecorator* Node)
{
	Node->DebuggerRuntimeDescription.Empty();
	for (int32 i = Node->FirstExecutionIndex; i <= Node->LastExecutionIndex; i++)
	{
		if (RuntimeDescriptions.IsValidIndex(i) && RuntimeDescriptions[i].Len())
		{
			if (Node->DebuggerRuntimeDescription.Len())
			{
				Node->DebuggerRuntimeDescription.AppendChar(TEXT('\n'));
			}

			Node->DebuggerRuntimeDescription += FString::Printf(TEXT("[%d] %s"), i, *RuntimeDescriptions[i].Replace(TEXT("\n"), TEXT(", ")));
		}
	}
}

void FBehaviorTreeDebugger::CollectBreakpointsFromAsset(class UBehaviorTreeGraphNode* Node)
{
	if (Node == NULL)
	{
		return;
	}

	for (int32 PinIdx = 0; PinIdx < Node->Pins.Num(); PinIdx++)
	{
		UEdGraphPin* Pin = Node->Pins[PinIdx];
		if (Pin->Direction != EGPD_Output)
		{
			continue;
		}

		for (int32 i = 0; i < Pin->LinkedTo.Num(); i++)
		{
			UBehaviorTreeGraphNode* LinkedNode = Cast<UBehaviorTreeGraphNode>(Pin->LinkedTo[i]->GetOwningNode());
			if (LinkedNode)
			{
				UBTNode* BTNode = Cast<UBTNode>(LinkedNode->NodeInstance);
				if (BTNode && LinkedNode->bHasBreakpoint && LinkedNode->bIsBreakpointEnabled)
				{
					ActiveBreakpoints.Add(BTNode->GetExecutionIndex());
				}

				CollectBreakpointsFromAsset(LinkedNode);
			}
		}
	}
}

int32 FBehaviorTreeDebugger::FindMatchingDebuggerStack(UBehaviorTreeComponent& TestInstance) const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (TestInstance.DebuggerSteps.Num())
	{
		const FBehaviorTreeExecutionStep& StepInfo = TestInstance.DebuggerSteps.Last();
		for (int32 i = 0; i < StepInfo.InstanceStack.Num(); i++)
		{
			if (StepInfo.InstanceStack[i].TreeAsset == TreeAsset)
			{
				return i;
			}
		}
	}
#endif

	return INDEX_NONE;
}

int32 FBehaviorTreeDebugger::FindDebuggerStepForTimeStamp(double Timestamp) const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (TreeInstance.IsValid())
	{
		for(int32 Idx = TreeInstance->DebuggerSteps.Num() - 1; Idx > 0; --Idx)
		{
			if (TreeInstance->DebuggerSteps[Idx].TimeStamp < Timestamp)
			{
				return Idx;
			}
		}
	}
#endif // USE_USE_BEHAVIORTREE_DEBUGGER
	return INDEX_NONE;
}

UBehaviorTreeComponent* FBehaviorTreeDebugger::FindInstanceInActor(AActor* TestActor)
{
	UBehaviorTreeComponent* FoundInstance = NULL;
	if (TestActor)
	{
		APawn* TestPawn = Cast<APawn>(TestActor);
		if (TestPawn && TestPawn->GetController())
		{
			FoundInstance = TestPawn->GetController()->FindComponentByClass<UBehaviorTreeComponent>();
		}

		if (FoundInstance == NULL)
		{
			FoundInstance = TestActor->FindComponentByClass<UBehaviorTreeComponent>();
		}
	}

	return FoundInstance;
}

void FBehaviorTreeDebugger::FindLockedDebugActor(UWorld* World)
{
	APlayerController* LocalPC = GEngine->GetFirstLocalPlayerController(World);
	if (LocalPC && LocalPC->GetHUD() && LocalPC->GetPawnOrSpectator())
	{
		APawn* SelectedPawn = NULL;
#if WITH_ENGINE
		const UEditorEngine* EEngine = Cast<UEditorEngine>(GEngine);
		for (FSelectionIterator It = EEngine->GetSelectedActorIterator(); It; ++It)
		{
			SelectedPawn = Cast<APawn>(*It);
			if (SelectedPawn)
			{
				break;
			}
		}
#endif //WITH_ENGINE

		UBehaviorTreeComponent* TestInstance = FindInstanceInActor((APawn*)SelectedPawn);
		if (TestInstance)
		{
			TreeInstance = TestInstance;
#if USE_BEHAVIORTREE_DEBUGGER
			ActiveStepIndex = TestInstance->DebuggerSteps.Num() - 1;
#endif
		}
	}
}

void FBehaviorTreeDebugger::FindMatchingTreeInstance()
{
	KnownInstances.Reset();

	// Find the world for the dedicated server if any, otherwise fallback to the PIE world
	UWorld* PlayWorld = nullptr;
	for (const FWorldContext& PieContext : GEditor->GetWorldContexts())
	{
		if (PieContext.WorldType == EWorldType::PIE && PieContext.World() != nullptr)
		{
			if (PieContext.RunAsDedicated)
			{
				PlayWorld = PieContext.World();
				break;
			}
			else if(!PlayWorld)
			{
				PlayWorld = PieContext.World();
				// Need to continue to see if their is a dedicated server.
			}
		}
	}
	 
	if (PlayWorld == NULL)
	{
		return;
	}

	UBehaviorTreeManager* BTManager = UBehaviorTreeManager::GetCurrent(PlayWorld);
	if (!BTManager)
	{
		return;
	}

	UBehaviorTreeComponent* MatchingComp = NULL;
	for (UBehaviorTreeComponent* ActiveBTComp : BTManager->GetActiveComponents())
	{
		AActor* TestActor = ActiveBTComp ? ActiveBTComp->GetOwner() : nullptr;
		if (ActiveBTComp && TestActor)
		{
			KnownInstances.Add(ActiveBTComp);

			const int32 MatchingIdx = FindMatchingDebuggerStack(*ActiveBTComp);
			if (MatchingIdx != INDEX_NONE)
			{
				MatchingComp = ActiveBTComp;

				if (TestActor->IsSelected())
				{
					TreeInstance = ActiveBTComp;
					return;
				}
			}
		}
	}

	if (MatchingComp != TreeInstance)
	{
		TreeInstance = MatchingComp;
		UpdateDebuggerViewOnInstanceChange();
	}
}

bool FBehaviorTreeDebugger::IsDebuggerReady() const
{
	return bIsPIEActive;
}

bool FBehaviorTreeDebugger::IsDebuggerRunning() const
{
	return TreeInstance.IsValid() && (ActiveStepIndex != INDEX_NONE);
}

bool FBehaviorTreeDebugger::IsShowingCurrentState() const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (TreeInstance.IsValid() && TreeInstance->DebuggerSteps.Num())
	{
		return (TreeInstance->DebuggerSteps.Num() - 1) == ActiveStepIndex;
	}
#endif

	return false;
}

int32 FBehaviorTreeDebugger::GetShownStateIndex() const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (TreeInstance.IsValid())
	{
		return (TreeInstance->DebuggerSteps.Num() - 1) - ActiveStepIndex;
	}
#endif

	return 0;
}

bool FBehaviorTreeDebugger::IsBehaviorExecutionPaused() const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (TreeInstance.IsValid() && TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex))
	{
		return TreeInstance->DebuggerSteps[ActiveStepIndex].bIsExecutionPaused;
	}
#endif
	return false;
}

void FBehaviorTreeDebugger::StepForwardInto()
{
#if USE_BEHAVIORTREE_DEBUGGER
	UpdateCurrentStep(ActiveStepIndex, StepForwardIntoIdx);
#endif
}

static void ForEachGameWorld(const TFunction<void(UWorld*)>& Func)
{
	for (const FWorldContext& PieContext : GUnrealEd->GetWorldContexts())
	{
		UWorld* PlayWorld = PieContext.World();
		if (PlayWorld && PlayWorld->IsGameWorld())
		{
			Func(PlayWorld);
		}
	}
}

static bool AreAllGameWorldPaused()
{
	bool bPaused = true;
	ForEachGameWorld([&](UWorld* World)
	{ 
		bPaused = bPaused && World->bDebugPauseExecution; 
	});
	return bPaused;
}

bool FBehaviorTreeDebugger::CanStepForwardInto() const
{
	return AreAllGameWorldPaused() && (StepForwardIntoIdx != INDEX_NONE);
}

void FBehaviorTreeDebugger::StepForwardOver()
{
#if USE_BEHAVIORTREE_DEBUGGER
	UpdateCurrentStep(ActiveStepIndex, StepForwardOverIdx);
#endif
}

bool FBehaviorTreeDebugger::CanStepForwardOver() const
{
	return AreAllGameWorldPaused() && (StepForwardOverIdx != INDEX_NONE);
}

void FBehaviorTreeDebugger::StepOut()
{
#if USE_BEHAVIORTREE_DEBUGGER
	UpdateCurrentStep(ActiveStepIndex, StepOutIdx);
#endif
}

void FBehaviorTreeDebugger::OpenParentTree()
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (ActiveStepIndex != INDEX_NONE)
	{
		if (TreeInstance.IsValid())
		{
			if (TSharedPtr<FBehaviorTreeEditor> Owner = EditorOwner.Pin())
			{
				Owner->DebuggerSwitchAsset(TreeInstance->DebuggerSteps[ActiveStepIndex].InstanceStack[DebuggerInstanceIndex - 1].TreeAsset);
			}
		}
	}
#endif // USE_BEHAVIORTREE_DEBUGGER
}

void FBehaviorTreeDebugger::OpenSubtree()
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (ActiveStepIndex != INDEX_NONE)
	{
		if (TreeInstance.IsValid())
		{
			if (TSharedPtr<FBehaviorTreeEditor> Owner = EditorOwner.Pin())
			{
				Owner->DebuggerSwitchAsset(TreeInstance->DebuggerSteps[ActiveStepIndex].InstanceStack[DebuggerInstanceIndex + 1].TreeAsset);
			}
		}
	}
#endif // USE_BEHAVIORTREE_DEBUGGER
}

bool FBehaviorTreeDebugger::CanStepOut() const
{
	return AreAllGameWorldPaused() && (StepOutIdx != INDEX_NONE);
}

bool FBehaviorTreeDebugger::CanOpenParentTree() const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (ActiveStepIndex != INDEX_NONE)
	{
		if (TreeInstance.IsValid())
		{
			return TreeInstance->DebuggerSteps[ActiveStepIndex].InstanceStack.IsValidIndex(DebuggerInstanceIndex - 1);
		}
	}
#endif // USE_BEHAVIORTREE_DEBUGGER
	return false;
}

bool FBehaviorTreeDebugger::CanOpenSubtree() const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (ActiveStepIndex != INDEX_NONE)
	{
		if (TreeInstance.IsValid())
		{
			return TreeInstance->DebuggerSteps[ActiveStepIndex].InstanceStack.IsValidIndex(DebuggerInstanceIndex + 1);
		}
	}
#endif // USE_BEHAVIORTREE_DEBUGGER
	return false;
}

void FBehaviorTreeDebugger::StepBackInto()
{
#if USE_BEHAVIORTREE_DEBUGGER
	UpdateCurrentStep(ActiveStepIndex, StepBackIntoIdx);
#endif
}

bool FBehaviorTreeDebugger::CanStepBackInto() const
{
	return AreAllGameWorldPaused() && (StepBackIntoIdx != INDEX_NONE);
}

void FBehaviorTreeDebugger::StepBackOver()
{
#if USE_BEHAVIORTREE_DEBUGGER
	UpdateCurrentStep(ActiveStepIndex, StepBackOverIdx);
#endif
}

bool FBehaviorTreeDebugger::CanStepBackOver() const
{
	return AreAllGameWorldPaused() && (StepBackOverIdx != INDEX_NONE);
}

void FBehaviorTreeDebugger::UpdateCurrentStep(int32 PrevStepIdx, int32 NewStepIdx, bool bSwitchAsset)
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (TreeInstance.IsValid() && TreeInstance->DebuggerSteps.IsValidIndex(NewStepIdx))
	{
		const int32 CurInstaceIdx = FindActiveInstanceIdx(PrevStepIdx);
		const int32 NewInstanceIdx = FindActiveInstanceIdx(NewStepIdx);

		const FBehaviorTreeExecutionStep& CurStepInfo = TreeInstance->DebuggerSteps[PrevStepIdx];
		const FBehaviorTreeExecutionStep& NewStepInfo = TreeInstance->DebuggerSteps[NewStepIdx];

		ActiveStepIndex = NewStepIdx;

		if (NewInstanceIdx != INDEX_NONE && NewStepInfo.InstanceStack[NewInstanceIdx].TreeAsset != TreeAsset)
		{
			if (CurInstaceIdx != NewInstanceIdx || 
				CurStepInfo.InstanceStack[CurInstaceIdx].TreeAsset != NewStepInfo.InstanceStack[NewInstanceIdx].TreeAsset)
			{
				if(bSwitchAsset)
				{
					if (TSharedPtr<FBehaviorTreeEditor> Owner = EditorOwner.Pin())
					{
						Owner->DebuggerSwitchAsset(NewStepInfo.InstanceStack[NewInstanceIdx].TreeAsset);
					}
				}
				UpdateCurrentSubtree();
			}
		}

		if (NewStepInfo.InstanceStack.IsValidIndex(DebuggerInstanceIndex))
		{
			const FBehaviorTreeDebuggerInstance& ShowInstance = NewStepInfo.InstanceStack[DebuggerInstanceIndex];
			UpdateAssetFlags(ShowInstance, RootNode.Get(), NewStepInfo.ExecutionStepId);
		}
		else
		{
			ActiveStepIndex = INDEX_NONE;

			FBehaviorTreeDebuggerInstance EmptyData;
			UpdateAssetFlags(EmptyData, RootNode.Get(), FBehaviorTreeExecutionStep::InvalidExecutionId);
		}

		UpdateDebuggerViewOnStepChange();
		UpdateAvailableActions();
	}
#endif
}

bool FBehaviorTreeDebugger::HasContinuousNextStep() const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (TreeInstance.IsValid() && TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex + 1))
	{
		const FBehaviorTreeExecutionStep& NextStepInfo = TreeInstance->DebuggerSteps[ActiveStepIndex + 1];
		const FBehaviorTreeExecutionStep& CurStepInfo = TreeInstance->DebuggerSteps[ActiveStepIndex];

		if (CurStepInfo.InstanceStack.IsValidIndex(DebuggerInstanceIndex) &&
			CurStepInfo.InstanceStack.Num() == NextStepInfo.InstanceStack.Num() &&
			CurStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset == NextStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset)
		{
			return true;
		}
	}
#endif
	return false;
}

bool FBehaviorTreeDebugger::HasContinuousPrevStep() const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (TreeInstance.IsValid() && TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex - 1))
	{
		const FBehaviorTreeExecutionStep& PrevStepInfo = TreeInstance->DebuggerSteps[ActiveStepIndex - 1];
		const FBehaviorTreeExecutionStep& CurStepInfo = TreeInstance->DebuggerSteps[ActiveStepIndex];

		if (CurStepInfo.InstanceStack.IsValidIndex(DebuggerInstanceIndex) &&
			CurStepInfo.InstanceStack.Num() == PrevStepInfo.InstanceStack.Num() &&
			CurStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset == PrevStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset)
		{
			return true;
		}
	}
#endif
	return false;
}

void FBehaviorTreeDebugger::OnActiveNodeChanged(const TArray<uint16>& ActivePath, const TArray<uint16>& PrevStepPath)
{
	StoppedOnBreakpointExecutionIndex = INDEX_NONE;
	if (ActiveBreakpoints.Num() == 0)
	{
		return;
	}

	bool bShouldPause = false;
	
	// breakpoints: check only nodes, that have changed from previous state
	// (e.g. breakpoint on sequence, it would break multiple times for every child
	// but we want only once: when it becomes active)

	for (int32 i = 0; i < ActivePath.Num(); i++)
	{
		const uint16 TestExecutionIndex = ActivePath[i];
		if (!PrevStepPath.Contains(TestExecutionIndex))
		{
			if (ActiveBreakpoints.Contains(TestExecutionIndex))
			{
				bShouldPause = true;
				StoppedOnBreakpointExecutionIndex = TestExecutionIndex;
				break;
			}
		}
	}

	if (bShouldPause)
	{
		if (TSharedPtr<FBehaviorTreeEditor> Owner = EditorOwner.Pin())
		{
			Owner->FocusWindow(TreeAsset);
		}

		PausePlaySession();
	}
}

void FBehaviorTreeDebugger::StopPlaySession()
{
	if (GUnrealEd->PlayWorld)
	{
		GEditor->RequestEndPlayMap();
 
		// @TODO: we need a unified flow to leave debugging mode from the different debuggers to prevent strong coupling between modules.
		// Each debugger (Blueprint & BehaviorTree for now) could then take the appropriate actions to resume the session.
		if (FSlateApplication::Get().InKismetDebuggingMode())
		{
			FSlateApplication::Get().LeaveDebuggingMode();
		}
	}
}

void FBehaviorTreeDebugger::PausePlaySession()
{
	if (GUnrealEd->SetPIEWorldsPaused(true))
	{
		GUnrealEd->PlaySessionPaused();
	}
}

void FBehaviorTreeDebugger::ResumePlaySession()
{
	if (GUnrealEd->SetPIEWorldsPaused(false))
	{
		// @TODO: we need a unified flow to leave debugging mode from the different debuggers to prevent strong coupling between modules.
		// Each debugger (Blueprint & BehaviorTree for now) could then take the appropriate actions to resume the session.
		if (FSlateApplication::Get().InKismetDebuggingMode())
		{
			FSlateApplication::Get().LeaveDebuggingMode();
		}

		GUnrealEd->PlaySessionResumed();
	}
}

bool FBehaviorTreeDebugger::IsPlaySessionPaused()
{
	return AreAllGameWorldPaused();
}

bool FBehaviorTreeDebugger::IsPlaySessionRunning()
{
	return !AreAllGameWorldPaused();
}

bool FBehaviorTreeDebugger::IsPIESimulating()
{
	return GEditor->bIsSimulatingInEditor || GEditor->PlayWorld;
}

bool FBehaviorTreeDebugger::IsPIENotSimulating()
{
	return !GEditor->bIsSimulatingInEditor && (GEditor->PlayWorld == NULL);
}

void FBehaviorTreeDebugger::OnBreakpointAdded(class UBehaviorTreeGraphNode* Node)
{
	if (IsDebuggerReady())
	{
		UBTNode* BTNode = Cast<UBTNode>(Node->NodeInstance);
		if (BTNode)
		{
			ActiveBreakpoints.AddUnique(BTNode->GetExecutionIndex());
		}
	}
}

void FBehaviorTreeDebugger::OnBreakpointRemoved(class UBehaviorTreeGraphNode* Node)
{
	if (IsDebuggerReady())
	{
		UBTNode* BTNode = Cast<UBTNode>(Node->NodeInstance);
		if (BTNode)
		{
			ActiveBreakpoints.RemoveSingleSwap(BTNode->GetExecutionIndex());
		}
	}
}

void FBehaviorTreeDebugger::UpdateDebuggerViewOnInstanceChange()
{
#if USE_BEHAVIORTREE_DEBUGGER
	UBlackboardData* BBAsset = nullptr;
	if (TSharedPtr<FBehaviorTreeEditor> Owner = EditorOwner.Pin())
	{
		BBAsset = Owner->GetBlackboardData();
	}

	if (TreeInstance.IsValid() &&
		TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex) &&
		TreeInstance->DebuggerSteps[ActiveStepIndex].InstanceStack.IsValidIndex(DebuggerInstanceIndex))
	{
		const FBehaviorTreeDebuggerInstance& ShowInstance = TreeInstance->DebuggerSteps[ActiveStepIndex].InstanceStack[DebuggerInstanceIndex];
		if (ShowInstance.TreeAsset)
		{
			BBAsset = ShowInstance.TreeAsset->BlackboardAsset;
		}
	}

	OnDebuggedBlackboardChangedEvent.Broadcast(BBAsset);

	if (DebuggerInstanceIndex != INDEX_NONE)
	{
		Refresh();
	}
	else
	{
		ClearDebuggerState(/*bKeepSubtreeData=*/true);
	}
#endif
}

void FBehaviorTreeDebugger::UpdateDebuggerViewOnStepChange()
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (IsDebuggerRunning() &&
		TreeInstance.IsValid() &&
		TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex))
	{
		const FBehaviorTreeExecutionStep& ShowStep = TreeInstance->DebuggerSteps[ActiveStepIndex];

		SavedTimestamp = ShowStep.TimeStamp;
		SavedValues = ShowStep.BlackboardValues;
	}
#endif
}

void FBehaviorTreeDebugger::UpdateDebuggerViewOnTick()
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (IsDebuggerRunning() && TreeInstance.IsValid())
	{
		const double GameTime = GEditor && GEditor->PlayWorld ? GEditor->PlayWorld->GetTimeSeconds() : 0.;
		CurrentTimestamp = GameTime;

		TreeInstance->StoreDebuggerBlackboard(CurrentValues);
	}
#endif
}

TOptional<FBehaviorTreeNodeDebugContext> FBehaviorTreeDebugger::GetDebugContextForNode(const UBTNode* Node) const
{
	if (UBehaviorTreeComponent* TreeComp = TreeInstance.Get())
	{
		if (TreeComp->DebuggerSteps.IsValidIndex(ActiveStepIndex))
		{
			const FBehaviorTreeExecutionStep& ActiveStep = TreeComp->DebuggerSteps[ActiveStepIndex];
			if (ActiveStep.InstanceStack.IsValidIndex(DebuggerInstanceIndex))
			{
				const FBehaviorTreeDebuggerInstance& DebugInstance = ActiveStep.InstanceStack[DebuggerInstanceIndex];
				if (DebugInstance.RuntimeDesc.IsValidIndex(Node->GetExecutionIndex()))
				{
					return FBehaviorTreeNodeDebugContext{ *TreeComp, DebugInstance.RuntimeDesc[Node->GetExecutionIndex()]};
				}
			}
		}
		return FBehaviorTreeNodeDebugContext{ *TreeComp, FString() };
	}
	return {};
}

FText FBehaviorTreeDebugger::FindValueForKey(const FName& InKeyName, bool bUseCurrentState) const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (IsDebuggerRunning() &&
		TreeInstance.IsValid())
	{
		const TMap<FName, FString>* MapToQuery = nullptr;
		if(bUseCurrentState)
		{
			MapToQuery = &CurrentValues;
		}
		else if(TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex))
		{
			MapToQuery = &TreeInstance->DebuggerSteps[ActiveStepIndex].BlackboardValues;
		}

		if(MapToQuery != nullptr)
		{
			const FString* FindValue = MapToQuery->Find(InKeyName);
			if(FindValue != nullptr)
			{
				return FText::FromString(*FindValue);
			}	
		}
	}

	return FText();
#endif
}

double FBehaviorTreeDebugger::GetTimeStamp(bool bUseCurrentState) const
{
	return bUseCurrentState ? CurrentTimestamp : SavedTimestamp;
}

FString FBehaviorTreeDebugger::GetDebuggedInstanceDesc() const
{
	UBehaviorTreeComponent* BTComponent = TreeInstance.Get();
	return BTComponent ? DescribeInstance(*BTComponent) : NSLOCTEXT("BlueprintEditor", "DebugActorNothingSelected", "No debug object selected").ToString();
}

FString FBehaviorTreeDebugger::DescribeInstance(UBehaviorTreeComponent& InstanceToDescribe) const
{
	FString ActorDesc;
	if (InstanceToDescribe.GetOwner())
	{
		AController* TestController = Cast<AController>(InstanceToDescribe.GetOwner());
		ActorDesc = TestController ?
			TestController->GetName() :
			InstanceToDescribe.GetOwner()->GetActorLabel();
	}

	return ActorDesc;
}

void FBehaviorTreeDebugger::OnInstanceSelectedInDropdown(UBehaviorTreeComponent* SelectedInstance)
{
	if (SelectedInstance)
	{
		ClearDebuggerState();

		AController* OldController = TreeInstance.IsValid() ? Cast<AController>(TreeInstance->GetOwner()) : NULL;
		APawn* OldPawn = OldController != NULL ? OldController->GetPawn().Get() : NULL;
		USelection* SelectedActors = GEditor ? GEditor->GetSelectedActors() : NULL;
		if (SelectedActors)
		{
			SelectedActors->DeselectAll();
		}

		TreeInstance = SelectedInstance;

		if (SelectedActors && SelectedInstance && SelectedInstance->GetOwner())
		{
			AController* TestController = Cast<AController>(SelectedInstance->GetOwner());
			APawn* Pawn = TestController != NULL ? TestController->GetPawn().Get() : NULL;
			if (Pawn)
			{
				SelectedActors->Select(Pawn);
			}
		}

		Refresh();
	}
}

void FBehaviorTreeDebugger::GetMatchingInstances(TArray<UBehaviorTreeComponent*>& MatchingInstances)
{
	for (int32 i = KnownInstances.Num() - 1; i >= 0; i--)
	{
		UBehaviorTreeComponent* TestInstance = KnownInstances[i].Get();
		if (TestInstance == NULL)
		{
			KnownInstances.RemoveAt(i);
			continue;
		}

		const int32 StackIdx = FindMatchingDebuggerStack(*TestInstance);
		if (StackIdx != INDEX_NONE)
		{
			MatchingInstances.Add(TestInstance);
		}
	}
}

void FBehaviorTreeDebugger::InitializeFromParent(class FBehaviorTreeDebugger* ParentDebugger)
{
	ClearDebuggerState();

#if USE_BEHAVIORTREE_DEBUGGER
	TreeInstance = ParentDebugger->TreeInstance;
	ActiveStepIndex = ParentDebugger->ActiveStepIndex;

	UpdateDebuggerInstance();
	UpdateAvailableActions();

	if (TreeInstance.IsValid() &&
		TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex))
	{
		FBehaviorTreeExecutionStep& ExecutionStep = TreeInstance->DebuggerSteps[ActiveStepIndex];
		if (ExecutionStep.InstanceStack.IsValidIndex(DebuggerInstanceIndex))
		{
			const FBehaviorTreeDebuggerInstance& ShowInstance = ExecutionStep.InstanceStack[DebuggerInstanceIndex];
			UpdateAssetFlags(ShowInstance, RootNode.Get(), ExecutionStep.ExecutionStepId);
		}
	}
#endif
}

int32 FBehaviorTreeDebugger::FindActiveInstanceIdx(int32 StepIdx) const
{
#if USE_BEHAVIORTREE_DEBUGGER
	const FBehaviorTreeExecutionStep& StepInfo = TreeInstance->DebuggerSteps[StepIdx];
	for (int32 i = StepInfo.InstanceStack.Num() - 1; i >= 0; i--)
	{
		if (StepInfo.InstanceStack[i].IsValid())
		{
			return i;
		}
	}
#endif

	return INDEX_NONE;
}

void FBehaviorTreeDebugger::UpdateCurrentSubtree()
{
	bIsCurrentSubtree = false;

#if USE_BEHAVIORTREE_DEBUGGER
	if (TreeInstance->DebuggerSteps.IsValidIndex(ActiveStepIndex))
	{
		const FBehaviorTreeExecutionStep& StepInfo = TreeInstance->DebuggerSteps[ActiveStepIndex];

		// assume that top instance is always valid, so it won't take away step buttons when tree is finished as out of nodes
		// current subtree = no child instances, or child instances are not valid
		bIsCurrentSubtree = ((DebuggerInstanceIndex == 0) || (StepInfo.InstanceStack.IsValidIndex(DebuggerInstanceIndex) && StepInfo.InstanceStack[DebuggerInstanceIndex].IsValid())) &&
			(!StepInfo.InstanceStack.IsValidIndex(DebuggerInstanceIndex + 1) || !StepInfo.InstanceStack[DebuggerInstanceIndex + 1].IsValid());
	}
#endif
}

static int32 GetNumActiveInstances(const FBehaviorTreeExecutionStep& StepInfo, class UBehaviorTree*& ActiveSubtree)
{
	ActiveSubtree = !StepInfo.InstanceStack.IsEmpty() ? StepInfo.InstanceStack.Last().TreeAsset : nullptr;
	return StepInfo.InstanceStack.Num();
}

void FBehaviorTreeDebugger::UpdateAvailableActions()
{
	StepForwardIntoIdx = INDEX_NONE;
	StepForwardOverIdx = INDEX_NONE;
	StepBackIntoIdx = INDEX_NONE;
	StepBackOverIdx = INDEX_NONE;
	StepOutIdx = INDEX_NONE;

#if USE_BEHAVIORTREE_DEBUGGER
	UBehaviorTreeComponent* TreeInstancePtr = TreeInstance.Get();
	if (TreeInstancePtr && TreeInstancePtr->DebuggerSteps.IsValidIndex(ActiveStepIndex) && DebuggerInstanceIndex >= 0)
	{
		const FBehaviorTreeExecutionStep& CurStepInfo = TreeInstancePtr->DebuggerSteps[ActiveStepIndex];

		if (TreeInstancePtr->DebuggerSteps.IsValidIndex(ActiveStepIndex - 1))
		{
			StepBackIntoIdx = ActiveStepIndex - 1;
		}

		if (TreeInstancePtr->DebuggerSteps.IsValidIndex(ActiveStepIndex + 1))
		{
			StepForwardIntoIdx = ActiveStepIndex + 1;
		}

		UBehaviorTree* CurTree = CurStepInfo.InstanceStack.IsValidIndex(DebuggerInstanceIndex) ?
			CurStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset : NULL;
		const int32 CurStepInstances = DebuggerInstanceIndex + 1;

		for (int32 TestStepIndex = ActiveStepIndex - 1; TestStepIndex >= 0; TestStepIndex--)
		{
			const FBehaviorTreeExecutionStep& TestStepInfo = TreeInstancePtr->DebuggerSteps[TestStepIndex];
			UBehaviorTree* TestTree = NULL;
			const int32 TestStepInstances = GetNumActiveInstances(TestStepInfo, TestTree);

			StepBackOverIdx = TestStepIndex;

			// keep going only if the execution is moving to a sub-tree
			if (TestStepInstances <= CurStepInstances ||
				TestStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset != CurStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset)
			{
				break;
			}
		}

		for (int32 TestStepIndex = ActiveStepIndex + 1; TestStepIndex < TreeInstancePtr->DebuggerSteps.Num(); TestStepIndex++)
		{
			const FBehaviorTreeExecutionStep& TestStepInfo = TreeInstancePtr->DebuggerSteps[TestStepIndex];
			UBehaviorTree* TestTree = NULL;
			int32 TestStepInstances = GetNumActiveInstances(TestStepInfo, TestTree);

			StepForwardOverIdx = TestStepIndex;

			// keep going only if the execution is moving to a sub-tree
			if (TestStepInstances <= CurStepInstances ||
				TestStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset != CurStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset)
			{
				break;
			}
		}

		if (CurStepInfo.InstanceStack.IsValidIndex(DebuggerInstanceIndex) && CurStepInfo.InstanceStack[DebuggerInstanceIndex].ActivePath.Num())
		{
			for (int32 TestStepIndex = ActiveStepIndex + 1; TestStepIndex < TreeInstancePtr->DebuggerSteps.Num(); TestStepIndex++)
			{
				const FBehaviorTreeExecutionStep& TestStepInfo = TreeInstancePtr->DebuggerSteps[TestStepIndex];
				UBehaviorTree* TestTree = NULL;
				int32 TestStepInstances = GetNumActiveInstances(TestStepInfo, TestTree);

				if (TestStepInstances < CurStepInstances ||
					TestStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset != CurStepInfo.InstanceStack[DebuggerInstanceIndex].TreeAsset)
				{
					// execution left current subtree
					StepOutIdx = TestStepIndex;
					break;
				}
			}
		}
	}
#endif
}
