// Copyright Epic Games, Inc. All Rights Reserved.
#include "BehaviorTreeDecoratorGraph.h"

#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTreeDecoratorGraphNode.h"
#include "BehaviorTreeDecoratorGraphNode_Decorator.h"
#include "BehaviorTreeDecoratorGraphNode_Logic.h"
#include "Containers/EnumAsByte.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_BehaviorTreeDecorator.h"
#include "HAL/PlatformCrt.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectPtr.h"

//////////////////////////////////////////////////////////////////////////
// BehaviorTreeGraph

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeDecoratorGraph)

UBehaviorTreeDecoratorGraph::UBehaviorTreeDecoratorGraph(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	Schema = UEdGraphSchema_BehaviorTreeDecorator::StaticClass();
}

void UBehaviorTreeDecoratorGraph::CollectDecoratorData(TArray<UBTDecorator*>& DecoratorInstances, TArray<FBTDecoratorLogic>& DecoratorOperations) const
{
	const UBehaviorTreeDecoratorGraphNode* RootNode = FindRootNode();
	CollectDecoratorDataWorker(RootNode, DecoratorInstances, DecoratorOperations);
}

const UBehaviorTreeDecoratorGraphNode* UBehaviorTreeDecoratorGraph::FindRootNode() const
{
	for (int32 i = 0; i < Nodes.Num(); i++)
	{
		const UBehaviorTreeDecoratorGraphNode_Logic* TestNode = Cast<const UBehaviorTreeDecoratorGraphNode_Logic>(Nodes[i]);
		if (TestNode && TestNode->LogicMode == EDecoratorLogicMode::Sink)
		{
			return TestNode;
		}
	}

	return NULL;
}

void UBehaviorTreeDecoratorGraph::CollectDecoratorDataWorker(const UBehaviorTreeDecoratorGraphNode* Node,
															 TArray<UBTDecorator*>& DecoratorInstances, TArray<FBTDecoratorLogic>& DecoratorOperations) const
{
	if (Node == NULL)
	{
		return;
	}

	TArray<const UBehaviorTreeDecoratorGraphNode*> LinkedNodes;
	for (int32 i = 0; i < Node->Pins.Num(); i++)
	{
		if (Node->Pins[i]->Direction == EGPD_Input &&
			Node->Pins[i]->LinkedTo.Num() > 0)
		{
			const UBehaviorTreeDecoratorGraphNode* LinkedNode = Cast<const UBehaviorTreeDecoratorGraphNode>(Node->Pins[i]->LinkedTo[0]->GetOwningNode());
			if (LinkedNode)
			{
				LinkedNodes.Add(LinkedNode);
			}
		}
	}

	FBTDecoratorLogic LogicOp(IntCastChecked<uint8>((int32)Node->GetOperationType()), IntCastChecked<uint16>(LinkedNodes.Num()));

	// special case: invalid
	if (LogicOp.Operation == EBTDecoratorLogic::Invalid)
	{
		// discard
	}
	// special case: test
	else if (LogicOp.Operation == EBTDecoratorLogic::Test)
	{
		// add decorator instance
		const UBehaviorTreeDecoratorGraphNode_Decorator* DecoratorNode = Cast<const UBehaviorTreeDecoratorGraphNode_Decorator>(Node);
		UBTDecorator* DecoratorInstance = DecoratorNode ? (UBTDecorator*)DecoratorNode->NodeInstance : NULL;
		if (DecoratorInstance)
		{
			LogicOp.Number = IntCastChecked<uint16>(DecoratorInstances.Add(DecoratorInstance));
			DecoratorOperations.Add(LogicOp);
		}
	}
	else
	{
		DecoratorOperations.Add(LogicOp);
	}

	for (int32 i = 0; i < LinkedNodes.Num(); i++)
	{
		CollectDecoratorDataWorker(LinkedNodes[i], DecoratorInstances, DecoratorOperations);
	}
}

UEdGraphPin* UBehaviorTreeDecoratorGraph::FindFreePin(UEdGraphNode* Node, EEdGraphPinDirection Direction)
{
	for (int32 Idx = 0; Idx < Node->Pins.Num(); Idx++)
	{
		if (Node->Pins[Idx] && Node->Pins[Idx]->Direction == Direction && Node->Pins[Idx]->LinkedTo.Num() == 0)
		{
			return Node->Pins[Idx];
		}
	}

	return nullptr;
}

UBehaviorTreeDecoratorGraphNode* UBehaviorTreeDecoratorGraph::SpawnMissingNodeWorker(const TArray<class UBTDecorator*>& NodeInstances, const TArray<struct FBTDecoratorLogic>& Operations, int32& Index, UEdGraphNode* ParentGraphNode, int32 ChildIdx)
{
	int32 NumSteps = 0;
	UBehaviorTreeDecoratorGraphNode* GraphNode = nullptr;
	UBehaviorTreeDecoratorGraphNode_Logic* LogicNode = nullptr;

	const FBTDecoratorLogic& Op = Operations[Index];
	Index++;

	if (Op.Operation == EBTDecoratorLogic::Test)
	{
		FGraphNodeCreator<UBehaviorTreeDecoratorGraphNode_Decorator> NodeBuilder(*this);
		UBehaviorTreeDecoratorGraphNode_Decorator* CastedGraphNode = NodeBuilder.CreateNode();
		NodeBuilder.Finalize();

		GraphNode = CastedGraphNode;
		CastedGraphNode->NodeInstance = NodeInstances[Op.Number];
	}
	else
	{
		FGraphNodeCreator<UBehaviorTreeDecoratorGraphNode_Logic> NodeBuilder(*this);
		LogicNode = NodeBuilder.CreateNode();
		LogicNode->LogicMode = LogicNode->GetLogicMode(Op.Operation);
		NodeBuilder.Finalize();

		GraphNode = LogicNode;
		NumSteps = Op.Number;
	}

	if (GraphNode)
	{
		GraphNode->NodePosX = ParentGraphNode->NodePosX - 300;
		GraphNode->NodePosY = ParentGraphNode->NodePosY + ChildIdx * 100;
	}

	for (int32 Idx = 0; Idx < NumSteps; Idx++)
	{
		UBehaviorTreeDecoratorGraphNode* ChildNode = SpawnMissingNodeWorker(NodeInstances, Operations, Index, GraphNode, Idx);

		UEdGraphPin* ChildOut = FindFreePin(ChildNode, EGPD_Output);
		UEdGraphPin* NodeIn = FindFreePin(GraphNode, EGPD_Input);
		
		if (NodeIn == nullptr && LogicNode)
		{
			NodeIn = LogicNode->AddInputPin();
		}

		if (NodeIn && ChildOut)
		{
			NodeIn->MakeLinkTo(ChildOut);
		}
	}

	return GraphNode;
}

int32 UBehaviorTreeDecoratorGraph::SpawnMissingNodes(const TArray<class UBTDecorator*>& NodeInstances, const TArray<struct FBTDecoratorLogic>& Operations, int32 StartIndex)
{
	UBehaviorTreeDecoratorGraphNode* RootNode = nullptr;
	for (int32 Idx = 0; Idx < Nodes.Num(); Idx++)
	{
		UBehaviorTreeDecoratorGraphNode_Logic* TestNode = Cast<UBehaviorTreeDecoratorGraphNode_Logic>(Nodes[Idx]);
		if (TestNode && TestNode->LogicMode == EDecoratorLogicMode::Sink)
		{
			RootNode = TestNode;
			break;
		}
	}

	int32 NextIndex = StartIndex;
	if (RootNode)
	{
		UBehaviorTreeDecoratorGraphNode* OperationRoot = SpawnMissingNodeWorker(NodeInstances, Operations, NextIndex, RootNode, 0);
		if (OperationRoot)
		{
			UEdGraphPin* RootIn = FindFreePin(RootNode, EGPD_Input);
			UEdGraphPin* OpOut = FindFreePin(OperationRoot, EGPD_Output);

			RootIn->MakeLinkTo(OpOut);
		}
	}
	else
	{
		NextIndex++;
	}

	return NextIndex;
}
