// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeDecoratorGraphNode.h"

#include "BehaviorTreeDecoratorGraph.h"
#include "BehaviorTreeGraphNode_CompositeDecorator.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_BehaviorTreeDecorator.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeDecoratorGraphNode)

UBehaviorTreeDecoratorGraphNode::UBehaviorTreeDecoratorGraphNode(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	bAllowModifingInputs = true;
}

UEdGraphPin* UBehaviorTreeDecoratorGraphNode::GetInputPin(int32 InputIndex) const
{
	check(InputIndex >= 0);

	for (int32 PinIndex = 0, FoundInputs = 0; PinIndex < Pins.Num(); PinIndex++)
	{
		if (Pins[PinIndex]->Direction == EGPD_Input)
		{
			if (InputIndex == FoundInputs)
			{
				return Pins[PinIndex];
			}
			else
			{
				FoundInputs++;
			}
		}
	}

	return NULL;
}

UEdGraphPin* UBehaviorTreeDecoratorGraphNode::GetOutputPin(int32 InputIndex) const
{
	check(InputIndex >= 0);

	for (int32 PinIndex = 0, FoundInputs = 0; PinIndex < Pins.Num(); PinIndex++)
	{
		if (Pins[PinIndex]->Direction == EGPD_Output)
		{
			if (InputIndex == FoundInputs)
			{
				return Pins[PinIndex];
			}
			else
			{
				FoundInputs++;
			}
		}
	}

	return NULL;
}

UBehaviorTreeDecoratorGraph* UBehaviorTreeDecoratorGraphNode::GetDecoratorGraph()
{
	return CastChecked<UBehaviorTreeDecoratorGraph>(GetGraph());
}

EBTDecoratorLogic::Type UBehaviorTreeDecoratorGraphNode::GetOperationType() const
{
	return EBTDecoratorLogic::Invalid;
}

bool UBehaviorTreeDecoratorGraphNode::CanUserDeleteNode() const
{
	const UBehaviorTreeDecoratorGraph* MyGraph = Cast<const UBehaviorTreeDecoratorGraph>(GetGraph());
	return MyGraph == NULL || MyGraph->bEditable;
}

void UBehaviorTreeDecoratorGraphNode::AutowireNewNode(UEdGraphPin* FromPin)
{
	Super::AutowireNewNode(FromPin);

	if (FromPin != NULL)
	{
		if (GetSchema()->TryCreateConnection(FromPin, FromPin->Direction == EGPD_Input ? GetOutputPin() : GetInputPin()))
		{
			FromPin->GetOwningNode()->NodeConnectionListChanged();
		}
	}
}

void UBehaviorTreeDecoratorGraphNode::NodeConnectionListChanged()
{
	Super::NodeConnectionListChanged();

	const UBehaviorTreeDecoratorGraph* MyGraph = CastChecked<UBehaviorTreeDecoratorGraph>(GetGraph());
	
	UBehaviorTreeGraphNode_CompositeDecorator* CompositeDecorator = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(MyGraph->GetOuter());
	CompositeDecorator->OnInnerGraphChanged();
}

bool UBehaviorTreeDecoratorGraphNode::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const
{
	return DesiredSchema->GetClass()->IsChildOf(UEdGraphSchema_BehaviorTreeDecorator::StaticClass());
}

