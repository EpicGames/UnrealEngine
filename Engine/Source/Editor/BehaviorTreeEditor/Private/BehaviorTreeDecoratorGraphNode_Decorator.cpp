// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeDecoratorGraphNode_Decorator.h"

#include "AIGraphModule.h"
#include "AIGraphNode.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/BTNode.h"
#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTreeDecoratorGraph.h"
#include "BehaviorTreeGraphNode_CompositeDecorator.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformMath.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/NumericLimits.h"
#include "Templates/Casts.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UObjectBaseUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeDecoratorGraphNode_Decorator)

UBehaviorTreeDecoratorGraphNode_Decorator::UBehaviorTreeDecoratorGraphNode_Decorator(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeInstance = NULL;
}

void UBehaviorTreeDecoratorGraphNode_Decorator::AllocateDefaultPins()
{
	CreatePin(EGPD_Output, TEXT("Transition"), TEXT("Out"));
}

void UBehaviorTreeDecoratorGraphNode_Decorator::PostPlacedNewNode()
{
	UClass* NodeClass = ClassData.GetClass(true);
	if (NodeClass != NULL)
	{
		UBehaviorTreeGraphNode_CompositeDecorator* OwningNode = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(GetDecoratorGraph()->GetOuter());

		if (!NodeClass->IsChildOf(UBTDecorator::StaticClass()))
		{
			UE_LOGF(LogAIGraph, Error, "Unexpected instance class:%ls in composite decorator[%d]: [%ls][%ls]",
				*GetNameSafe(NodeClass),
				OwningNode ? OwningNode->FirstExecutionIndex : -1,
				OwningNode ? *OwningNode->GetNodeTitle(ENodeTitleType::FullTitle).ToString() : TEXT("??"),
				OwningNode ? *OwningNode->GetDescription().ToString() : TEXT("??"));

			return;
		}

		if (OwningNode)
		{
			UBehaviorTree* BT = Cast<UBehaviorTree>(OwningNode->GetOuter()->GetOuter());
			if (BT)
			{
				UBTDecorator* MyDecorator = NewObject<UBTDecorator>(BT, NodeClass);
				MyDecorator->InitializeFromAsset(*BT);
				OwningNode->InitializeDecorator(MyDecorator);

				NodeInstance = MyDecorator;
			}
		}
	}
}

FText UBehaviorTreeDecoratorGraphNode_Decorator::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	const UBTNode* MyNode = Cast<UBTNode>(NodeInstance);
	if (MyNode != NULL)
	{
		return FText::FromString(MyNode->GetStaticDescription());
	}

	return Super::GetNodeTitle(TitleType);
}

EBTDecoratorLogic::Type UBehaviorTreeDecoratorGraphNode_Decorator::GetOperationType() const
{
	return EBTDecoratorLogic::Test;
}

void UBehaviorTreeDecoratorGraphNode_Decorator::PrepareForCopying()
{
	if (NodeInstance)
	{
		// Temporarily take ownership of the node instance, so that it is not deleted when cutting
		NodeInstance->Rename(NULL, this, REN_DontCreateRedirectors | REN_DoNotDirty );
	}
}

void UBehaviorTreeDecoratorGraphNode_Decorator::PostEditImport()
{
	ResetNodeOwner();

	if (NodeInstance)
	{
		UBehaviorTreeGraphNode_CompositeDecorator* OwningNode = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(GetDecoratorGraph()->GetOuter());
		if (OwningNode)
		{
			UBehaviorTree* BT = Cast<UBehaviorTree>(OwningNode->GetOuter()->GetOuter());
			if (BT)
			{
				UBTDecorator* MyDecorator = (UBTDecorator*)NodeInstance;
				MyDecorator->InitializeFromAsset(*BT);
				MyDecorator->InitializeNode(NULL, MAX_uint16, 0, 0);

				OwningNode->InitializeDecorator(MyDecorator);
			}
		}
	}
}

void UBehaviorTreeDecoratorGraphNode_Decorator::PostEditUndo()
{
	Super::PostEditUndo();
	ResetNodeOwner();
}

void UBehaviorTreeDecoratorGraphNode_Decorator::PostCopyNode()
{
	ResetNodeOwner();
}

void UBehaviorTreeDecoratorGraphNode_Decorator::ResetNodeOwner()
{
	if (NodeInstance)
	{
		UBehaviorTreeGraphNode_CompositeDecorator* OwningNode = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(GetDecoratorGraph()->GetOuter());
		UBehaviorTree* BT = OwningNode ? Cast<UBehaviorTree>(OwningNode->GetOuter()->GetOuter()) : NULL;

		NodeInstance->Rename(NULL, BT, REN_DontCreateRedirectors | REN_DoNotDirty);
		NodeInstance->ClearFlags(RF_Transient);
	}
}

bool UBehaviorTreeDecoratorGraphNode_Decorator::RefreshNodeClass()
{
	bool bUpdated = false;
	if (NodeInstance == NULL)
	{
		if (FGraphNodeClassHelper::IsClassKnown(ClassData))
		{
			PostPlacedNewNode();
			bUpdated = (NodeInstance != NULL);
		}
		else
		{
			FGraphNodeClassHelper::AddUnknownClass(ClassData);
		}
	}

	return bUpdated;
}

void UBehaviorTreeDecoratorGraphNode_Decorator::UpdateNodeClassData()
{
	if (NodeInstance)
	{
		UAIGraphNode::UpdateNodeClassDataFrom(NodeInstance->GetClass(), ClassData);
	}
}
