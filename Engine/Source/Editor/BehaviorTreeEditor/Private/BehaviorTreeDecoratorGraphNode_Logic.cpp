// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeDecoratorGraphNode_Logic.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_BehaviorTreeDecorator.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeDecoratorGraphNode_Logic)

UBehaviorTreeDecoratorGraphNode_Logic::UBehaviorTreeDecoratorGraphNode_Logic(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

bool UBehaviorTreeDecoratorGraphNode_Logic::CanUserDeleteNode() const
{
	return LogicMode != EDecoratorLogicMode::Sink;
}

void UBehaviorTreeDecoratorGraphNode_Logic::AllocateDefaultPins()
{
	AddInputPin();
	if (LogicMode == EDecoratorLogicMode::And || LogicMode == EDecoratorLogicMode::Or)
	{
		AddInputPin();
	}

	if (LogicMode != EDecoratorLogicMode::Sink)
	{
		CreatePin(EGPD_Output, TEXT("Transition"), TEXT("Out"));
	}
}

static FString DescribeLogicModeHelper(const EDecoratorLogicMode::Type& Mode)
{
	FString Desc[] = { TEXT("Result"), TEXT("AND"), TEXT("OR"), TEXT("NOT") };
	return Desc[Mode];
}

FText UBehaviorTreeDecoratorGraphNode_Logic::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::FromString(DescribeLogicModeHelper(LogicMode));
}

void UBehaviorTreeDecoratorGraphNode_Logic::GetMenuEntries(struct FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	EDecoratorLogicMode::Type Modes[] = { EDecoratorLogicMode::And, EDecoratorLogicMode::Or, EDecoratorLogicMode::Not };
	const int32 NumModes = sizeof(Modes) / sizeof(Modes[0]);
	for (int32 i = 0; i < NumModes; i++)
	{
		TSharedPtr<FDecoratorSchemaAction_NewNode> AddOpAction = UEdGraphSchema_BehaviorTreeDecorator::AddNewDecoratorAction(ContextMenuBuilder, NSLOCTEXT("BehaviorTreeGraphNode_Logic", "Logic", "Logic"), FText::FromString(DescribeLogicModeHelper(Modes[i])), FText::GetEmpty());

		UBehaviorTreeDecoratorGraphNode_Logic* OpNode = NewObject<UBehaviorTreeDecoratorGraphNode_Logic>(ContextMenuBuilder.OwnerOfTemporaries);
		OpNode->LogicMode = Modes[i];
		AddOpAction->NodeTemplate = OpNode;
	}
}

bool UBehaviorTreeDecoratorGraphNode_Logic::CanAddPins() const
{
	return (LogicMode == EDecoratorLogicMode::And || LogicMode == EDecoratorLogicMode::Or);
}

bool UBehaviorTreeDecoratorGraphNode_Logic::CanRemovePins() const
{
	int32 NumInputLinks = 0;
	for (int32 i = 0; i < Pins.Num(); i++)
	{
		if (Pins[i]->Direction == EGPD_Input)
		{
			NumInputLinks++;
		}
	}

	return (NumInputLinks > 2) && CanAddPins();
}

UEdGraphPin* UBehaviorTreeDecoratorGraphNode_Logic::AddInputPin()
{
	return CreatePin(EGPD_Input, TEXT("Transition"), TEXT("In"));
}

void UBehaviorTreeDecoratorGraphNode_Logic::RemoveInputPin(class UEdGraphPin* Pin)
{
	Pin->MarkAsGarbage();
	Pins.Remove(Pin);
}

EBTDecoratorLogic::Type UBehaviorTreeDecoratorGraphNode_Logic::GetOperationType() const
{
	EBTDecoratorLogic::Type LogicTypes[] = { EBTDecoratorLogic::Invalid, EBTDecoratorLogic::And, EBTDecoratorLogic::Or, EBTDecoratorLogic::Not };
	return LogicTypes[LogicMode];
}

EDecoratorLogicMode::Type UBehaviorTreeDecoratorGraphNode_Logic::GetLogicMode(EBTDecoratorLogic::Type Op) const
{
	EDecoratorLogicMode::Type LogicTypes[] = { EDecoratorLogicMode::Sink, EDecoratorLogicMode::Sink, EDecoratorLogicMode::And, EDecoratorLogicMode::Or, EDecoratorLogicMode::Not };
	return LogicTypes[Op];
}
