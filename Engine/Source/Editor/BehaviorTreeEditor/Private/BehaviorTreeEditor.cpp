// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeEditor.h"

#include "AIGraphNode.h"
#include "AIGraphTypes.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTree/BehaviorTreeComponent.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTree/BlackboardData.h"
#include "BehaviorTree/Decorators/BTDecorator_BlueprintBase.h"
#include "BehaviorTree/Services/BTService_BlueprintBase.h"
#include "BehaviorTree/Tasks/BTTask_BlueprintBase.h"
#include "BehaviorTree/Tasks/BTTask_RunBehavior.h"
#include "BehaviorTree/Tasks/BTTask_RunBehaviorDynamic.h"
#include "BehaviorTreeColors.h"
#include "BehaviorTreeDebugger.h"
#include "BehaviorTreeDecoratorGraphNode_Logic.h"
#include "BehaviorTreeEditorCommands.h"
#include "BehaviorTreeEditorModes.h"
#include "BehaviorTreeEditorModule.h"
#include "BehaviorTreeEditorTabFactories.h"
#include "BehaviorTreeEditorTabs.h"
#include "BehaviorTreeEditorToolbar.h"
#include "BehaviorTreeEditorTypes.h"
#include "BehaviorTreeEditorUtils.h"
#include "BehaviorTreeGraph.h"
#include "BehaviorTreeGraphNode.h"
#include "BehaviorTreeGraphNode_CompositeDecorator.h"
#include "BehaviorTreeGraphNode_Decorator.h"
#include "BehaviorTreeGraphNode_Root.h"
#include "BehaviorTreeGraphNode_SubtreeTask.h"
#include "BlackboardDataFactory.h"
#include "BlueprintUtilities.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "ContentBrowserModule.h"
#include "Delegates/Delegate.h"
#include "DetailCustomizations/BlackboardDataDetails.h"
#include "DetailsViewArgs.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_BehaviorTree.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/World.h"
#include "FindInBT.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "GraphEditorActions.h"
#include "HAL/PlatformCrt.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Layout/Margin.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "SBehaviorTreeBlackboardEditor.h"
#include "SBehaviorTreeBlackboardView.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UnrealEdGlobals.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"

class SWidget;

#define LOCTEXT_NAMESPACE "BehaviorTreeEditor"

const FName FBehaviorTreeEditor::BehaviorTreeMode(TEXT("BehaviorTree"));
const FName FBehaviorTreeEditor::BlackboardMode(TEXT("Blackboard"));

FText FBehaviorTreeEditor::BehaviorTreeModeText(LOCTEXT("BehaviorTreeMode", "Behavior Tree"));
FText FBehaviorTreeEditor::BlackboardModeText(LOCTEXT("BlackboardMode", "Blackboard"));

//////////////////////////////////////////////////////////////////////////
FBehaviorTreeEditor::FBehaviorTreeEditor() 
	: IBehaviorTreeEditor()
{
	// listen for package change events to update injected nodes
	OnPackageSavedDelegateHandle     = UPackage::PackageSavedWithContextEvent.AddRaw(this, &FBehaviorTreeEditor::OnPackageSaved);

	bShowDecoratorRangeLower = false;
	bShowDecoratorRangeSelf = false;
	bSelectedNodeIsInjected = false;
	bSelectedNodeIsRootLevel = false;

	bHasMultipleTaskBP = false;
	bHasMultipleDecoratorBP = false;
	bHasMultipleServiceBP = false;

	BehaviorTree = nullptr;
	BlackboardData = nullptr;

	bCheckDirtyOnAssetSave = true;

	GraphClass = UBehaviorTreeGraph::StaticClass();
	GraphName = "Behavior Tree";
	CornerText = LOCTEXT("AppearanceCornerText", "BEHAVIOR TREE");
	TitleText = LOCTEXT("BehaviorTreeGraphLabel", "Behavior Tree");
	RootNodeNoteText = LOCTEXT("RootLevelNode", "Root-level decorators are only valid and will be executed if this BT is be used\nas static a sub-tree (via \"Run Behavior\"). These decorators will be ignored if\ndynamically injected with \"Run Dynamic Behavior\".");
}

FBehaviorTreeEditor::~FBehaviorTreeEditor()
{
	UPackage::PackageSavedWithContextEvent.Remove(OnPackageSavedDelegateHandle);

	Debugger.Reset();
}

void FBehaviorTreeEditor::RefreshBlackboardViewsAssociatedObject()
{
	if (BlackboardView.IsValid())
	{
		BlackboardView->SetObject(GetBlackboardData());
	}

	if (BlackboardEditor.IsValid())
	{
		BlackboardEditor->SetObject(GetBlackboardData());
	}
}

void FBehaviorTreeEditor::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		RefreshBlackboardViewsAssociatedObject();
	}

	FAIGraphEditor::PostUndo(bSuccess);
}

void FBehaviorTreeEditor::PostRedo(bool bSuccess)
{
	if (bSuccess)
	{
		RefreshBlackboardViewsAssociatedObject();
	}

	FAIGraphEditor::PostRedo(bSuccess);
}

void FBehaviorTreeEditor::NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged )
{
	if(PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		if(PropertyChangedEvent.Property != nullptr && PropertyChangedEvent.Property->GetFName() == TEXT("BlackboardAsset"))
		{
			BlackboardData = BehaviorTree->BlackboardAsset;
		}

		RefreshBlackboardViewsAssociatedObject();
	}
}

void FBehaviorTreeEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	DocumentManager->SetTabManager(InTabManager);

	FWorkflowCentricApplication::RegisterTabSpawners(InTabManager);
}

void FBehaviorTreeEditor::RegisterToolbarTab(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
}

void FBehaviorTreeEditor::InitBehaviorTreeEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* InObject )
{
	UBehaviorTree* BehaviorTreeToEdit = Cast<UBehaviorTree>(InObject);
	UBlackboardData* BlackboardDataToEdit = Cast<UBlackboardData>(InObject);

	if(BehaviorTreeToEdit != nullptr)
	{
		BehaviorTree = BehaviorTreeToEdit;
		if(BehaviorTree->BlackboardAsset != nullptr)
		{
			BlackboardData = BehaviorTree->BlackboardAsset;
		}
	}
	else if(BlackboardDataToEdit != nullptr)
	{
		BlackboardData = BlackboardDataToEdit;
	}

	TSharedPtr<FBehaviorTreeEditor> ThisPtr(SharedThis(this));
	if(!DocumentManager.IsValid())
	{
		DocumentManager = MakeShareable(new FDocumentTracker);
		DocumentManager->Initialize(ThisPtr);

		// Register the document factories
		{
			TSharedRef<FDocumentTabFactory> GraphEditorFactory = MakeShareable(new FBTGraphEditorSummoner(ThisPtr,
				FBTGraphEditorSummoner::FOnCreateGraphEditorWidget::CreateSP(this, &FBehaviorTreeEditor::CreateGraphEditorWidget)
				));

			// Also store off a reference to the grapheditor factory so we can find all the tabs spawned by it later.
			GraphEditorTabFactoryPtr = GraphEditorFactory;
			DocumentManager->RegisterDocumentFactory(GraphEditorFactory);
		}
	}

	TArray<UObject*> ObjectsToEdit;
	if(BehaviorTree != nullptr)
	{
		ObjectsToEdit.Add(BehaviorTree);
	}
	if(BlackboardData != nullptr)
	{
		ObjectsToEdit.Add(BlackboardData);
	}

	if (!ToolbarBuilder.IsValid())
	{
		ToolbarBuilder = MakeShareable(new FBehaviorTreeEditorToolbar(SharedThis(this)));
	}

	// if we are already editing objects, dont try to recreate the editor from scratch but update the list of objects in edition
    // ex: BehaviorTree may want to reuse an editor already opened for its associated Blackboard asset.
	const TArray<UObject*>* EditedObjects = GetObjectsCurrentlyBeingEdited();
	if(EditedObjects == nullptr || EditedObjects->Num() == 0)
	{
		FGraphEditorCommands::Register();
		FBTCommonCommands::Register();
		FBTDebuggerCommands::Register();
		FBTBlackboardCommands::Register();

		const bool bCreateDefaultStandaloneMenu = true;
		const bool bCreateDefaultToolbar = true;
		InitAssetEditor( Mode, InitToolkitHost, FBehaviorTreeEditorModule::BehaviorTreeEditorAppIdentifier, FTabManager::FLayout::NullLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectsToEdit );

		BindCommonCommands();
		ExtendMenu();
		CreateInternalWidgets();

		Debugger = MakeShareable(new FBehaviorTreeDebugger);
		Debugger->Setup(BehaviorTree, SharedThis(this));
		BindDebuggerToolbarCommands();

		FBehaviorTreeEditorModule& BehaviorTreeEditorModule = FModuleManager::LoadModuleChecked<FBehaviorTreeEditorModule>( "BehaviorTreeEditor" );
		AddMenuExtender(BehaviorTreeEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
		AddToolbarExtender(BehaviorTreeEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

		AddApplicationMode(BehaviorTreeMode, MakeShareable(new FBehaviorTreeEditorApplicationMode(SharedThis(this))));
		AddApplicationMode(BlackboardMode, MakeShareable(new FBlackboardEditorApplicationMode(SharedThis(this))));

		BlackboardView = SNew(SBehaviorTreeBlackboardView, GetToolkitCommands(), GetBlackboardData())
			.OnGetDebugKeyValue(this, &FBehaviorTreeEditor::HandleGetDebugKeyValue)
			.OnIsDebuggerReady(this, &FBehaviorTreeEditor::IsDebuggerReady)
			.OnIsDebuggerPaused(this, &FBehaviorTreeEditor::IsDebuggerPaused)
			.OnGetDebugTimeStamp(this, &FBehaviorTreeEditor::HandleGetDebugTimeStamp)
			.OnGetDisplayCurrentState(this, &FBehaviorTreeEditor::HandleGetDisplayCurrentState);

		BlackboardEditor = SNew(SBehaviorTreeBlackboardEditor, GetToolkitCommands(), GetBlackboardData())
				.OnEntrySelected(this, &FBehaviorTreeEditor::HandleBlackboardEntrySelected)
				.OnGetDebugKeyValue(this, &FBehaviorTreeEditor::HandleGetDebugKeyValue)
				.OnIsDebuggerReady(this, &FBehaviorTreeEditor::IsDebuggerReady)
				.OnIsDebuggerPaused(this, &FBehaviorTreeEditor::IsDebuggerPaused)
				.OnGetDebugTimeStamp(this, &FBehaviorTreeEditor::HandleGetDebugTimeStamp)
				.OnGetDisplayCurrentState(this, &FBehaviorTreeEditor::HandleGetDisplayCurrentState)
				.OnIsBlackboardModeActive(this, &FBehaviorTreeEditor::HandleIsBlackboardModeActive);
	}
	else
	{
		check(Debugger.IsValid());
		Debugger->Setup(BehaviorTree, SharedThis(this));

		for (UObject* ObjectToEdit : ObjectsToEdit)
		{
			if (!EditedObjects->Contains(ObjectToEdit))
		    {
				AddEditingObject(ObjectToEdit);
		    }
		}
	}

	if(BehaviorTreeToEdit != nullptr)
	{
		SetCurrentMode(BehaviorTreeMode);
	}
	else if(BlackboardDataToEdit != nullptr)
	{
		SetCurrentMode(BlackboardMode);
	}

	OnClassListUpdated();
	RegenerateMenusAndToolbars();
}

void FBehaviorTreeEditor::RestoreBehaviorTree()
{
	// Update BT asset data based on saved graph to have correct data in editor
	UBehaviorTreeGraph* MyGraph = Cast<UBehaviorTreeGraph>(BehaviorTree->BTGraph);
	const bool bNewGraph = MyGraph == NULL;
	if (MyGraph == NULL)
	{
		const TSubclassOf<UEdGraphSchema> SchemaClass = GetDefault<UBehaviorTreeGraph>(GraphClass)->Schema;
		check(SchemaClass);
		BehaviorTree->BTGraph = FBlueprintEditorUtils::CreateNewGraph(BehaviorTree, GraphName, GraphClass, SchemaClass);
		MyGraph = Cast<UBehaviorTreeGraph>(BehaviorTree->BTGraph);

		// Initialize the behavior tree graph
		const UEdGraphSchema* Schema = MyGraph->GetSchema();
		Schema->CreateDefaultNodesForGraph(*MyGraph);

		MyGraph->OnCreated();
	}
	else
	{
		MyGraph->OnLoaded();
	}

	MyGraph->Initialize();

	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(MyGraph);
	TSharedPtr<SDockTab> DocumentTab = DocumentManager->OpenDocument(Payload, bNewGraph ? FDocumentTracker::OpenNewDocument : FDocumentTracker::RestorePreviousDocument);

	if(BehaviorTree->LastEditedDocuments.Num() > 0)
	{
		TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(DocumentTab->GetContent());
		GraphEditor->SetViewLocation(BehaviorTree->LastEditedDocuments[0].SavedViewOffset, BehaviorTree->LastEditedDocuments[0].SavedZoomAmount);
	}

	const bool bIncreaseVersionNum = false;
	if(bNewGraph)
	{
		MyGraph->UpdateAsset(UBehaviorTreeGraph::ClearDebuggerFlags | UBehaviorTreeGraph::KeepRebuildCounter);
		RefreshBlackboardViewsAssociatedObject();
	}
	else
	{
		MyGraph->UpdateAsset(UBehaviorTreeGraph::KeepRebuildCounter);
		RefreshDebugger();
	}

	FAbortDrawHelper EmptyMode;
	bShowDecoratorRangeLower = false;
	bShowDecoratorRangeSelf = false;
	bSelectedNodeIsInjected = false;
	bSelectedNodeIsRootLevel = false;
	MyGraph->UpdateAbortHighlight(EmptyMode, EmptyMode);
}

void FBehaviorTreeEditor::SaveEditedObjectState()
{
	// Clear currently edited documents
	BehaviorTree->LastEditedDocuments.Empty();

	// Ask all open documents to save their state, which will update LastEditedDocuments
	DocumentManager->SaveAllState();
}

void FBehaviorTreeEditor::HandleBlackboardEntrySelected(const FBlackboardEntry* BlackboardEntry, bool bIsInherited)
{
	// refresh details view
	const bool bForceRefresh = true;

	if (ensure(BlackboardDetailsView.IsValid()))
	{
		// the opposite should never happen, we weren't able to internally repro it, but it seems someone was crashing on this line
		BlackboardDetailsView->SetObject(GetBlackboardData(), bForceRefresh);
	}
}

int32 FBehaviorTreeEditor::HandleGetSelectedBlackboardItemIndex(bool& bIsInherited)
{
	if(BlackboardEditor.IsValid())
	{
		return BlackboardEditor->GetSelectedEntryIndex(bIsInherited);
	}

	return INDEX_NONE;
}

FText FBehaviorTreeEditor::HandleGetDebugKeyValue(const FName& InKeyName, bool bUseCurrentState) const
{
	if(IsDebuggerReady())
	{
		return Debugger->FindValueForKey(InKeyName, bUseCurrentState);
	}

	return FText();
}

double FBehaviorTreeEditor::HandleGetDebugTimeStamp(bool bUseCurrentState) const
{
	if(IsDebuggerReady())
	{
		return Debugger->GetTimeStamp(bUseCurrentState);
	}

	return 0.0f;
}

void FBehaviorTreeEditor::HandleDebuggedBlackboardChanged(UBlackboardData* InBlackboardData)
{
	RefreshBlackboardViewsAssociatedObject();
}

bool FBehaviorTreeEditor::HandleGetDisplayCurrentState() const
{
	if(IsDebuggerReady())
	{
		return Debugger->IsShowingCurrentState();
	}

	return false;
}

bool FBehaviorTreeEditor::HandleIsBlackboardModeActive() const
{
	return GetCurrentMode() == BlackboardMode;
}

void FBehaviorTreeEditor::GetBlackboardSelectionInfo(int32& OutSelectionIndex, bool& bOutIsInherited) const
{
	OutSelectionIndex = CurrentBlackboardEntryIndex;
	bOutIsInherited = bIsCurrentBlackboardEntryInherited;
}

bool FBehaviorTreeEditor::IsDebuggerReady() const
{
	return Debugger.IsValid() && Debugger->IsDebuggerReady();
}

bool FBehaviorTreeEditor::IsDebuggerPaused() const
{
	return IsDebuggerReady() && GUnrealEd->PlayWorld && GUnrealEd->PlayWorld->bDebugPauseExecution;
}

bool FBehaviorTreeEditor::CanEditWithDebuggerActive() const
{
	if(Debugger.IsValid())
	{
		return !Debugger->IsDebuggerReady();
	}

	return true;
}

EVisibility FBehaviorTreeEditor::GetDebuggerDetailsVisibility() const
{
	return Debugger.IsValid() && Debugger->IsDebuggerRunning() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FBehaviorTreeEditor::GetRangeLowerVisibility() const
{
	return FBehaviorTreeDebugger::IsPIENotSimulating() && bShowDecoratorRangeLower ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FBehaviorTreeEditor::GetRangeSelfVisibility() const
{
	return FBehaviorTreeDebugger::IsPIENotSimulating() && bShowDecoratorRangeSelf ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FBehaviorTreeEditor::GetInjectedNodeVisibility() const
{
	return FBehaviorTreeDebugger::IsPIENotSimulating() && bSelectedNodeIsInjected ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FBehaviorTreeEditor::GetRootLevelNodeVisibility() const
{
	return FBehaviorTreeDebugger::IsPIENotSimulating() && bSelectedNodeIsRootLevel ? EVisibility::Visible : EVisibility::Collapsed;
}

FGraphAppearanceInfo FBehaviorTreeEditor::GetGraphAppearance() const
{
	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = CornerText;

	const int32 StepIdx = Debugger.IsValid() ? Debugger->GetShownStateIndex() : 0;
	if (Debugger.IsValid() && !Debugger->IsDebuggerRunning())
	{
		AppearanceInfo.PIENotifyText = LOCTEXT("InactiveLabel", "INACTIVE");
	}
	else if (StepIdx)
	{
		AppearanceInfo.PIENotifyText = FText::Format(LOCTEXT("StepsBackLabelFmt", "STEPS BACK: {0}"), FText::AsNumber(StepIdx));
	}
	else if (FBehaviorTreeDebugger::IsPlaySessionPaused())
	{
		AppearanceInfo.PIENotifyText = LOCTEXT("PausedLabel", "PAUSED");
	}

	if (Debugger.IsValid() && Debugger->IsBehaviorExecutionPaused())
	{
		AppearanceInfo.WarningText = LOCTEXT("BehaviorExecutionPausedLabel", "BEHAVIOR EXECUTION PAUSED");
	}
	
	return AppearanceInfo;
}

FName FBehaviorTreeEditor::GetToolkitFName() const
{
	return FName("Behavior Tree");
}

FText FBehaviorTreeEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "BehaviorTree");
}

FString FBehaviorTreeEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "BehaviorTree ").ToString();
}


FLinearColor FBehaviorTreeEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.0f, 0.0f, 0.2f, 0.5f );
}


/** Create new tab for the supplied graph - don't call this directly, call SExplorer->FindTabForGraph.*/
TSharedRef<SGraphEditor> FBehaviorTreeEditor::CreateGraphEditorWidget(UEdGraph* InGraph)
{
	check(InGraph != NULL);
	
	if (!GraphEditorCommands.IsValid())
	{
		CreateCommandList();

		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().RemoveExecutionPin,
			FExecuteAction::CreateSP( this, &FBehaviorTreeEditor::OnRemoveInputPin ),
			FCanExecuteAction::CreateSP( this, &FBehaviorTreeEditor::CanRemoveInputPin )
			);

		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AddExecutionPin,
			FExecuteAction::CreateSP( this, &FBehaviorTreeEditor::OnAddInputPin ),
			FCanExecuteAction::CreateSP( this, &FBehaviorTreeEditor::CanAddInputPin )
			);

		// Debug actions
		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AddBreakpoint,
			FExecuteAction::CreateSP( this, &FBehaviorTreeEditor::OnAddBreakpoint ),
			FCanExecuteAction::CreateSP( this, &FBehaviorTreeEditor::CanAddBreakpoint ),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP( this, &FBehaviorTreeEditor::CanAddBreakpoint )
			);

		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().RemoveBreakpoint,
			FExecuteAction::CreateSP( this, &FBehaviorTreeEditor::OnRemoveBreakpoint ),
			FCanExecuteAction::CreateSP( this, &FBehaviorTreeEditor::CanRemoveBreakpoint ),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP( this, &FBehaviorTreeEditor::CanRemoveBreakpoint )
			);

		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().EnableBreakpoint,
			FExecuteAction::CreateSP( this, &FBehaviorTreeEditor::OnEnableBreakpoint ),
			FCanExecuteAction::CreateSP( this, &FBehaviorTreeEditor::CanEnableBreakpoint ),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP( this, &FBehaviorTreeEditor::CanEnableBreakpoint )
			);

		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().DisableBreakpoint,
			FExecuteAction::CreateSP( this, &FBehaviorTreeEditor::OnDisableBreakpoint ),
			FCanExecuteAction::CreateSP( this, &FBehaviorTreeEditor::CanDisableBreakpoint ),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP( this, &FBehaviorTreeEditor::CanDisableBreakpoint )
			);

		GraphEditorCommands->MapAction( FGraphEditorCommands::Get().ToggleBreakpoint,
			FExecuteAction::CreateSP( this, &FBehaviorTreeEditor::OnToggleBreakpoint ),
			FCanExecuteAction::CreateSP( this, &FBehaviorTreeEditor::CanToggleBreakpoint ),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP( this, &FBehaviorTreeEditor::CanToggleBreakpoint )
			);
	}

	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FBehaviorTreeEditor::OnSelectedNodesChanged);
	InEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &FBehaviorTreeEditor::OnNodeDoubleClicked);
	InEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &FBehaviorTreeEditor::OnNodeTitleCommitted);

	// Make title bar
	TSharedRef<SWidget> TitleBarWidget = 
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush( TEXT("Graph.TitleBackground") ) )
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.FillWidth(1.f)
			[
				SNew(STextBlock)
				.Text(TitleText)
				.TextStyle( FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText") )
			]
		];

	// Make full graph editor
	const bool bGraphIsEditable = InGraph->bEditable;
	return SNew(SGraphEditor)
		.AdditionalCommands(GraphEditorCommands)
		.IsEditable(this, &FBehaviorTreeEditor::InEditingMode, bGraphIsEditable)
		.Appearance(this, &FBehaviorTreeEditor::GetGraphAppearance)
		.TitleBar(TitleBarWidget)
		.GraphToEdit(InGraph)
		.GraphEvents(InEvents)
		.AutoExpandActionMenu(true);
}

bool FBehaviorTreeEditor::InEditingMode(bool bGraphIsEditable) const
{
	return bGraphIsEditable && FBehaviorTreeDebugger::IsPIENotSimulating();
}

TSharedRef<SWidget> FBehaviorTreeEditor::SpawnSearch()
{
	FindResults = SNew(SFindInBT, SharedThis(this));
	return FindResults.ToSharedRef();
}

TSharedRef<SWidget> FBehaviorTreeEditor::SpawnProperties()
{
	return
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		.HAlign(HAlign_Fill)
		[
			DetailsView.ToSharedRef()
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 5.0f)
			[
				SNew(SBorder)
				.BorderBackgroundColor(BehaviorTreeColors::NodeBody::InjectedSubNode)
				.BorderImage(FAppStyle::GetBrush("Graph.StateNode.Body"))
				.Visibility(this, &FBehaviorTreeEditor::GetInjectedNodeVisibility)
				.Padding(FMargin(5.0f))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("InjectedNode", "Node is injected by subtree and can't be edited"))
				]
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 5.0f)
			[
				SNew(SBorder)
				.BorderBackgroundColor(BehaviorTreeColors::NodeBody::InjectedSubNode)
				.BorderImage(FAppStyle::GetBrush("Graph.StateNode.Body"))
				.Visibility(this, &FBehaviorTreeEditor::GetRootLevelNodeVisibility)
				.Padding(FMargin(5.0f))
				[
					SNew(STextBlock)
					.Text(RootNodeNoteText)
				]
			]
			+SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 5.0f)
			[
				SNew(SBorder)
				.BorderBackgroundColor(BehaviorTreeColors::NodeBorder::HighlightAbortRange0)
				.BorderImage( FAppStyle::GetBrush( "Graph.StateNode.Body" ) )
				.Visibility(this, &FBehaviorTreeEditor::GetRangeLowerVisibility)
				.Padding(FMargin(5.0f))
				[
					SNew(STextBlock)
					.Text(FText::Format(LOCTEXT("AbortModeHighlight", "Nodes aborted by mode: {0}"), LOCTEXT("AbortPriorityLower", "Lower Priority")))
				]
			]
			+SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 5.0f)
			[
				SNew(SBorder)
				.BorderBackgroundColor(BehaviorTreeColors::NodeBorder::HighlightAbortRange1)
				.BorderImage( FAppStyle::GetBrush( "Graph.StateNode.Body" ) )
				.Visibility(this, &FBehaviorTreeEditor::GetRangeSelfVisibility)
				.Padding(FMargin(5.0f))
				[
					SNew(STextBlock)
					.Text(FText::Format(LOCTEXT("AbortModeHighlight", "Nodes aborted by mode: {0}"), LOCTEXT("AbortPrioritySelf", "Self")))
				]
			]
		];
}

TSharedRef<SWidget> FBehaviorTreeEditor::SpawnBlackboardView()
{
	return BlackboardView.ToSharedRef();
}

TSharedRef<SWidget> FBehaviorTreeEditor::SpawnBlackboardEditor()
{
	return BlackboardEditor.ToSharedRef();
}

TSharedRef<SWidget> FBehaviorTreeEditor::SpawnBlackboardDetails()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = this;
	BlackboardDetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );

	UBlackboardData* BBData = GetBlackboardData();
	FOnGetSelectedBlackboardItemIndex OnGetSelectedBlackboardItemIndex = FOnGetSelectedBlackboardItemIndex::CreateSP(this, &FBehaviorTreeEditor::HandleGetSelectedBlackboardItemIndex);
	FOnGetDetailCustomizationInstance LayoutVariableDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FBlackboardDataDetails::MakeInstance, OnGetSelectedBlackboardItemIndex, BBData);
	BlackboardDetailsView->RegisterInstancedCustomPropertyLayout(UBlackboardData::StaticClass(), LayoutVariableDetails);

	if (BBData)
	{
		BBData->UpdateDeprecatedKeys();
	}

	BlackboardDetailsView->SetObject(BBData);
	BlackboardDetailsView->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FBehaviorTreeEditor::CanEditWithDebuggerActive)));

	return BlackboardDetailsView.ToSharedRef();
}

void FBehaviorTreeEditor::OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection)
{
	BehaviorTreeEditorUtils::FPropertySelectionInfo SelectionInfo;
	TArray<UObject*> Selection = BehaviorTreeEditorUtils::GetSelectionForPropertyEditor(NewSelection, SelectionInfo);

	UBehaviorTreeGraph* MyGraph = Cast<UBehaviorTreeGraph>(BehaviorTree->BTGraph);
	FAbortDrawHelper Mode0, Mode1;
	bShowDecoratorRangeLower = false;
	bShowDecoratorRangeSelf = false;
	bForceDisablePropertyEdit = SelectionInfo.bInjectedNode;
	bSelectedNodeIsInjected = SelectionInfo.bInjectedNode;
	bSelectedNodeIsRootLevel = SelectionInfo.bRootLevelNode;

	if (Selection.Num() == 1)
	{
		if (DetailsView.IsValid())
		{
			DetailsView->SetObjects(Selection);
		}

		if (SelectionInfo.FoundDecorator)
		{
			GetAbortModePreview(SelectionInfo.FoundDecorator, Mode0, Mode1);
		}
		else if (SelectionInfo.FoundGraphNode_CompDecorator)
		{
			GetAbortModePreview(SelectionInfo.FoundGraphNode_CompDecorator, Mode0, Mode1);
		}
	}
	else if (DetailsView.IsValid())
	{
		if (Selection.Num() == 0)
		{
			// if nothing is selected, display the root
			UBehaviorTreeGraphNode* RootNode = nullptr;
			for (const auto& Node : MyGraph->Nodes)
			{
				RootNode = Cast<UBehaviorTreeGraphNode_Root>(Node);
				if (RootNode != nullptr)
				{
					break;
				}
			}

			DetailsView->SetObject(RootNode);
		}
		else
		{
			DetailsView->SetObject(nullptr);
		}
	}

	MyGraph->UpdateAbortHighlight(Mode0, Mode1);
}

static uint16 GetMaxAllowedRange(const class UBTDecorator* DecoratorOb)
{
	uint16 MaxRange = MAX_uint16;
	
	UBTCompositeNode* TestParent = DecoratorOb->GetParentNode();
	while (TestParent)
	{
		if (TestParent->IsApplyingDecoratorScope())
		{
			MaxRange = TestParent->GetLastExecutionIndex();
			break;
		}

		TestParent = TestParent->GetParentNode();
	}

	return MaxRange;
}

static void FillAbortPreview_LowerPriority(const class UBTDecorator* DecoratorOb, const UBTCompositeNode* DecoratorParent, FAbortDrawHelper& Mode)
{
	const uint16 MaxRange = GetMaxAllowedRange(DecoratorOb);

	Mode.AbortStart = DecoratorParent->GetChildExecutionIndex(DecoratorOb->GetChildIndex() + 1);
	Mode.AbortEnd = MaxRange;
	Mode.SearchStart = DecoratorParent->GetExecutionIndex();
	Mode.SearchEnd = MaxRange;
}

static void FillAbortPreview_Self(const class UBTDecorator* DecoratorOb, const UBTCompositeNode* DecoratorParent, FAbortDrawHelper& Mode)
{
	const uint16 MaxRange = GetMaxAllowedRange(DecoratorOb);

	Mode.AbortStart = DecoratorOb->GetExecutionIndex();
	Mode.AbortEnd = DecoratorParent->GetChildExecutionIndex(DecoratorOb->GetChildIndex() + 1) - 1;
	Mode.SearchStart = DecoratorParent->GetChildExecutionIndex(DecoratorOb->GetChildIndex() + 1);
	Mode.SearchEnd = MaxRange;
}

void FBehaviorTreeEditor::GetAbortModePreview(const class UBehaviorTreeGraphNode_CompositeDecorator* Node, struct FAbortDrawHelper& Mode0, struct FAbortDrawHelper& Mode1)
{
	Mode0.SearchStart = MAX_uint16;
	Mode0.AbortStart = MAX_uint16;
	Mode1.SearchStart = MAX_uint16;
	Mode1.AbortStart = MAX_uint16;

	TArray<UBTDecorator*> Decorators;
	TArray<FBTDecoratorLogic> Operations;
	Node->CollectDecoratorData(Decorators, Operations);

	int32 LowerPriIdx = INDEX_NONE;
	int32 SelfIdx = INDEX_NONE;
	
	for (int32 i = 0; i < Decorators.Num(); i++)
	{
		EBTFlowAbortMode::Type FlowAbort = Decorators[i]->GetParentNode() ? Decorators[i]->GetFlowAbortMode() : EBTFlowAbortMode::None;

		if (FlowAbort == EBTFlowAbortMode::LowerPriority || FlowAbort == EBTFlowAbortMode::Both)
		{
			LowerPriIdx = i;
		}
		
		if (FlowAbort == EBTFlowAbortMode::Self || FlowAbort == EBTFlowAbortMode::Both)
		{
			SelfIdx = i;
		}
	}

	if (Decorators.IsValidIndex(LowerPriIdx))
	{
		FillAbortPreview_LowerPriority(Decorators[LowerPriIdx], Decorators[LowerPriIdx]->GetParentNode(), Mode0);
		bShowDecoratorRangeLower = true;
	}

	if (Decorators.IsValidIndex(SelfIdx))
	{
		FillAbortPreview_Self(Decorators[SelfIdx], Decorators[SelfIdx]->GetParentNode(), Mode1);
		bShowDecoratorRangeSelf = true;
	}
}

void FBehaviorTreeEditor::GetAbortModePreview(const class UBTDecorator* DecoratorOb, FAbortDrawHelper& Mode0, FAbortDrawHelper& Mode1)
{
	const UBTCompositeNode* DecoratorParent = DecoratorOb ? DecoratorOb->GetParentNode() : NULL;
	EBTFlowAbortMode::Type FlowAbort = DecoratorParent ? DecoratorOb->GetFlowAbortMode() : EBTFlowAbortMode::None;

	Mode0.SearchStart = MAX_uint16;
	Mode0.AbortStart = MAX_uint16;
	Mode1.SearchStart = MAX_uint16;
	Mode1.AbortStart = MAX_uint16;

	switch (FlowAbort)
	{
		case EBTFlowAbortMode::LowerPriority:
			FillAbortPreview_LowerPriority(DecoratorOb, DecoratorParent, Mode0);
			bShowDecoratorRangeLower = true;
			break;

		case EBTFlowAbortMode::Self:
			FillAbortPreview_Self(DecoratorOb, DecoratorParent, Mode1);
			bShowDecoratorRangeSelf = true;
			break;

		case EBTFlowAbortMode::Both:
			FillAbortPreview_LowerPriority(DecoratorOb, DecoratorParent, Mode0);
			FillAbortPreview_Self(DecoratorOb, DecoratorParent, Mode1);
			bShowDecoratorRangeLower = true;
			bShowDecoratorRangeSelf = true;
			break;

		default:
			break;
	}
}

void FBehaviorTreeEditor::CreateInternalWidgets()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	FDetailsViewArgs DetailsViewArgs; 
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.NotifyHook = this;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	DetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );
	DetailsView->SetObject( NULL );
	DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &FBehaviorTreeEditor::IsPropertyEditable));
	DetailsView->OnFinishedChangingProperties().AddSP(this, &FBehaviorTreeEditor::OnFinishedChangingProperties);
}

void FBehaviorTreeEditor::ExtendMenu()
{
	struct Local
	{
		static void FillEditMenu(FMenuBuilder& MenuBuilder)
		{
			MenuBuilder.BeginSection("EditSearch", LOCTEXT("EditMenu_SearchHeading", "Search"));
			{
				MenuBuilder.AddMenuEntry(FBTCommonCommands::Get().SearchBT);
			}
			MenuBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> MenuExtender = MakeShareable(new FExtender);

	// Extend the Edit menu
	MenuExtender->AddMenuExtension(
		"EditHistory",
		EExtensionHook::After,
		GetToolkitCommands(),
		FMenuExtensionDelegate::CreateStatic(&Local::FillEditMenu));

	AddMenuExtender(MenuExtender);
}

void FBehaviorTreeEditor::BindCommonCommands()
{
	ToolkitCommands->MapAction(FBTCommonCommands::Get().SearchBT,
			FExecuteAction::CreateSP(this, &FBehaviorTreeEditor::SearchTree),
			FCanExecuteAction::CreateSP(this, &FBehaviorTreeEditor::CanSearchTree)
			);

	ToolkitCommands->MapAction(FBTCommonCommands::Get().NewBlackboard,
			FExecuteAction::CreateSP(this, &FBehaviorTreeEditor::CreateNewBlackboard),
			FCanExecuteAction::CreateSP(this, &FBehaviorTreeEditor::CanCreateNewBlackboard)
			);
}

void FBehaviorTreeEditor::SearchTree()
{
	TabManager->TryInvokeTab(FBehaviorTreeEditorTabs::SearchID);
	FindResults->FocusForUse();
}

bool FBehaviorTreeEditor::CanSearchTree() const
{
	return true;
}

TSharedRef<class SWidget> FBehaviorTreeEditor::OnGetDebuggerActorsMenu()
{
	FMenuBuilder MenuBuilder(true, NULL);

	if (Debugger.IsValid())
	{
		TArray<UBehaviorTreeComponent*> MatchingInstances;
		Debugger->GetMatchingInstances(MatchingInstances);

		// Fill the combo menu with presets of common screen resolutions
		for (int32 i = 0; i < MatchingInstances.Num(); i++)
		{
			if (MatchingInstances[i])
			{
				const FText ActorDesc = FText::FromString(Debugger->DescribeInstance(*MatchingInstances[i]));
				TWeakObjectPtr<UBehaviorTreeComponent> InstancePtr = MatchingInstances[i];

				FUIAction ItemAction(FExecuteAction::CreateSP(this, &FBehaviorTreeEditor::OnDebuggerActorSelected, InstancePtr));
				MenuBuilder.AddMenuEntry(ActorDesc, TAttribute<FText>(), FSlateIcon(), ItemAction);
			}
		}

		// Failsafe when no actor match
		if (MatchingInstances.Num() == 0)
		{
			const FText ActorDesc = LOCTEXT("NoMatchForDebug","Can't find matching actors");
			TWeakObjectPtr<UBehaviorTreeComponent> InstancePtr;

			FUIAction ItemAction(FExecuteAction::CreateSP(this, &FBehaviorTreeEditor::OnDebuggerActorSelected, InstancePtr));
			MenuBuilder.AddMenuEntry(ActorDesc, TAttribute<FText>(), FSlateIcon(), ItemAction);
		}
	}

	return MenuBuilder.MakeWidget();
}

void FBehaviorTreeEditor::OnDebuggerActorSelected(TWeakObjectPtr<UBehaviorTreeComponent> InstanceToDebug)
{
	if (Debugger.IsValid())
	{
		Debugger->OnInstanceSelectedInDropdown(InstanceToDebug.Get());
	}
}

FText FBehaviorTreeEditor::GetDebuggerActorDesc() const
{
	return Debugger.IsValid() ? FText::FromString(Debugger->GetDebuggedInstanceDesc()) : FText::GetEmpty();
}

void FBehaviorTreeEditor::BindDebuggerToolbarCommands()
{
	const FBTDebuggerCommands& Commands = FBTDebuggerCommands::Get();
	TSharedRef<FBehaviorTreeDebugger> DebuggerOb = Debugger.ToSharedRef();

	ToolkitCommands->MapAction(
		Commands.BackOver,
		FExecuteAction::CreateSP(DebuggerOb, &FBehaviorTreeDebugger::StepBackOver),
		FCanExecuteAction::CreateSP(DebuggerOb, &FBehaviorTreeDebugger::CanStepBackOver));

	ToolkitCommands->MapAction(
		Commands.BackInto,
		FExecuteAction::CreateSP(DebuggerOb, &FBehaviorTreeDebugger::StepBackInto),
		FCanExecuteAction::CreateSP(DebuggerOb, &FBehaviorTreeDebugger::CanStepBackInto));

	ToolkitCommands->MapAction(
		Commands.ForwardInto,
		FExecuteAction::CreateSP(DebuggerOb, &FBehaviorTreeDebugger::StepForwardInto),
		FCanExecuteAction::CreateSP(DebuggerOb, &FBehaviorTreeDebugger::CanStepForwardInto));

	ToolkitCommands->MapAction(
		Commands.ForwardOver,
		FExecuteAction::CreateSP(DebuggerOb, &FBehaviorTreeDebugger::StepForwardOver),
		FCanExecuteAction::CreateSP(DebuggerOb, &FBehaviorTreeDebugger::CanStepForwardOver));

	ToolkitCommands->MapAction(
		Commands.StepOut,
		FExecuteAction::CreateSP(DebuggerOb, &FBehaviorTreeDebugger::StepOut),
		FCanExecuteAction::CreateSP(DebuggerOb, &FBehaviorTreeDebugger::CanStepOut));

	ToolkitCommands->MapAction(
		Commands.OpenParentTree,
		FExecuteAction::CreateSP(DebuggerOb, &FBehaviorTreeDebugger::OpenParentTree),
		FCanExecuteAction::CreateSP(DebuggerOb, &FBehaviorTreeDebugger::CanOpenParentTree));

	ToolkitCommands->MapAction(
		Commands.OpenSubtree,
		FExecuteAction::CreateSP(DebuggerOb, &FBehaviorTreeDebugger::OpenSubtree),
		FCanExecuteAction::CreateSP(DebuggerOb, &FBehaviorTreeDebugger::CanOpenSubtree));

	ToolkitCommands->MapAction(
		Commands.PausePlaySession,
		FExecuteAction::CreateStatic(&FBehaviorTreeDebugger::PausePlaySession),
		FCanExecuteAction::CreateStatic(&FBehaviorTreeDebugger::IsPlaySessionRunning),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FBehaviorTreeDebugger::IsPlaySessionRunning));

	ToolkitCommands->MapAction(
		Commands.ResumePlaySession,
		FExecuteAction::CreateStatic(&FBehaviorTreeDebugger::ResumePlaySession),
		FCanExecuteAction::CreateStatic(&FBehaviorTreeDebugger::IsPlaySessionPaused),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FBehaviorTreeDebugger::IsPlaySessionPaused));

	ToolkitCommands->MapAction(
		Commands.StopPlaySession,
		FExecuteAction::CreateStatic(&FBehaviorTreeDebugger::StopPlaySession));
}

bool FBehaviorTreeEditor::IsPropertyEditable() const
{
	if (FBehaviorTreeDebugger::IsPIESimulating() || bForceDisablePropertyEdit)
	{
		return false;
	}

	TSharedPtr<SGraphEditor> FocusedGraphEd = UpdateGraphEdPtr.Pin();
	return FocusedGraphEd.IsValid() && FocusedGraphEd->GetCurrentGraph() && FocusedGraphEd->GetCurrentGraph()->bEditable;
}

void FBehaviorTreeEditor::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (FocusedGraphOwner.IsValid())
	{
		FocusedGraphOwner->OnInnerGraphChanged();
	}

	// update abort range highlight when changing decorator's flow abort mode
	if (PropertyChangedEvent.Property)
	{
		if(PropertyChangedEvent.Property->GetFName() == TEXT("FlowAbortMode"))
		{
			bShowDecoratorRangeLower = false;
			bShowDecoratorRangeSelf = false;

			FGraphPanelSelectionSet CurrentSelection = GetSelectedNodes();
			if (CurrentSelection.Num() == 1)
			{
				for (FGraphPanelSelectionSet::TConstIterator It(CurrentSelection); It; ++It)
				{
					UBehaviorTreeGraphNode_Decorator* DecoratorNode = Cast<UBehaviorTreeGraphNode_Decorator>(*It);
					if (DecoratorNode)
					{
						FAbortDrawHelper Mode0, Mode1;
						GetAbortModePreview(Cast<UBTDecorator>(DecoratorNode->NodeInstance), Mode0, Mode1);

						UBehaviorTreeGraph* MyGraph = Cast<UBehaviorTreeGraph>(BehaviorTree->BTGraph);
						MyGraph->UpdateAbortHighlight(Mode0, Mode1);
					}
				}			
			}
		}
		else if(PropertyChangedEvent.Property->GetFName() == TEXT("BlackboardAsset"))
		{
			if(BlackboardView.IsValid())
			{
				BlackboardView->SetObject(GetBlackboardData());
			}
		}
	}

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == TEXT("BehaviorAsset"))
	{
		UBehaviorTreeGraph* MyGraph = Cast<UBehaviorTreeGraph>(BehaviorTree->BTGraph);
		MyGraph->UpdateInjectedNodes();
		MyGraph->UpdateAsset(UBehaviorTreeGraph::ClearDebuggerFlags);
	}

	const TSharedPtr<SGraphEditor> FocusedGraphEd = UpdateGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid() && FocusedGraphEd->GetCurrentGraph())
	{
		FocusedGraphEd->GetCurrentGraph()->GetSchema()->ForceVisualizationCacheClear();
	}
}

void FBehaviorTreeEditor::OnPackageSaved(const FString& PackageFileName, UPackage* Package, FObjectPostSaveContext ObjectSaveContext)
{
	UBehaviorTreeGraph* MyGraph = BehaviorTree ? Cast<UBehaviorTreeGraph>(BehaviorTree->BTGraph) : NULL;
	if (MyGraph)
	{
		const bool bUpdated = MyGraph->UpdateInjectedNodes();
		if (bUpdated)
		{
			MyGraph->UpdateAsset(UBehaviorTreeGraph::ClearDebuggerFlags);
		}
	}
}

void FBehaviorTreeEditor::OnClassListUpdated()
{
	FAIGraphEditor::OnClassListUpdated();

	const int32 NumTaskBP = FGraphNodeClassHelper::GetObservedBlueprintClassCount(UBTTask_BlueprintBase::StaticClass());
	const int32 NumDecoratorBP = FGraphNodeClassHelper::GetObservedBlueprintClassCount(UBTDecorator_BlueprintBase::StaticClass());
	const int32 NumServiceBP = FGraphNodeClassHelper::GetObservedBlueprintClassCount(UBTService_BlueprintBase::StaticClass());

	bHasMultipleTaskBP = NumTaskBP > 1;
	bHasMultipleDecoratorBP = NumDecoratorBP > 1;
	bHasMultipleServiceBP = NumServiceBP > 1;
}

void FBehaviorTreeEditor::OnNodeDoubleClicked(class UEdGraphNode* Node)
{
	UBehaviorTreeGraphNode* MyNode = Cast<UBehaviorTreeGraphNode>(Node);
	if (MyNode && MyNode->bInjectedNode)
	{
		UBTTask_RunBehavior* SubtreeTask = MyNode->ParentNode ? Cast<UBTTask_RunBehavior>(MyNode->ParentNode->NodeInstance) : NULL;
		if (SubtreeTask && SubtreeTask->GetSubtreeAsset())
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(SubtreeTask->GetSubtreeAsset());

			IBehaviorTreeEditor* ChildNodeEditor = static_cast<IBehaviorTreeEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(SubtreeTask->GetSubtreeAsset(), true));
			if (ChildNodeEditor)
			{
				ChildNodeEditor->InitializeDebuggerState(Debugger.Get());
				
				int32 FirstInjectedIdx = INDEX_NONE;
				UBehaviorTreeGraphNode* MyParentNode = Cast<UBehaviorTreeGraphNode>(MyNode->ParentNode);
				for (int32 Idx = 0; Idx < MyParentNode->Decorators.Num(); Idx++)
				{
					if (MyParentNode->Decorators[Idx]->bInjectedNode)
					{
						FirstInjectedIdx = Idx;
						break;
					}
				}

				if (FirstInjectedIdx != INDEX_NONE)
				{
					const int32 NodeIdx = MyParentNode->Decorators.IndexOfByKey(MyNode) - FirstInjectedIdx;
					UEdGraphNode* OtherNode = ChildNodeEditor->FindInjectedNode(NodeIdx);
					if (OtherNode)
					{
						ChildNodeEditor->DoubleClickNode(OtherNode);
					}
				}
			}
		}
	}
	else if (UBehaviorTreeGraphNode_CompositeDecorator* Decorator = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(Node))
	{
		if (Decorator->GetBoundGraph())
		{
			TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(const_cast<UEdGraph*>(Decorator->GetBoundGraph()));
			
			TArray< TSharedPtr<SDockTab> > MatchingTabs;
			DocumentManager->FindMatchingTabs(Payload, MatchingTabs);
			if (MatchingTabs.Num())
			{
				DocumentManager->CloseTab(Payload);
				DocumentManager->OpenDocument(Payload, FDocumentTracker::RestorePreviousDocument);
			}
			else
			{
				DocumentManager->OpenDocument(Payload, FDocumentTracker::OpenNewDocument);
			}
		}
	}
	else if (MyNode)
	{
		if (const UBTNode* NodeInstance = Cast<UBTNode>(MyNode->NodeInstance))
		{
			const TOptional<FBehaviorTreeNodeDebugContext> DebugContext = Debugger ? Debugger->GetDebugContextForNode(NodeInstance) : TOptional<FBehaviorTreeNodeDebugContext>{};
			if (const UObject* AssetToOpen = NodeInstance->GetAssociatedAsset(DebugContext))
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AssetToOpen);
				if (const UBehaviorTree* OpenedBehaviorTree = Cast<UBehaviorTree>(AssetToOpen))
				{
					if (IBehaviorTreeEditor* ChildNodeEditor = static_cast<IBehaviorTreeEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(const_cast<UBehaviorTree*>(OpenedBehaviorTree), true)))
					{
						ChildNodeEditor->InitializeDebuggerState(Debugger.Get());
					}
				}
			}
		}
	}
}

void FBehaviorTreeEditor::OnAddInputPin()
{
	FGraphPanelSelectionSet CurrentSelection;
	TSharedPtr<SGraphEditor> FocusedGraphEd = UpdateGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		CurrentSelection = FocusedGraphEd->GetSelectedNodes();
	}

	// Iterate over all nodes, and add the pin
	for (FGraphPanelSelectionSet::TConstIterator It(CurrentSelection); It; ++It)
	{
		UBehaviorTreeDecoratorGraphNode_Logic* LogicNode = Cast<UBehaviorTreeDecoratorGraphNode_Logic>(*It);
		if (LogicNode)
		{
			const FScopedTransaction Transaction( LOCTEXT("AddInputPin", "Add Input Pin") );

			LogicNode->Modify();
			LogicNode->AddInputPin();

			const UEdGraphSchema* Schema = LogicNode->GetSchema();
			Schema->ReconstructNode(*LogicNode);
		}
	}

	// Refresh the current graph, so the pins can be updated
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->NotifyGraphChanged();
	}
}

bool FBehaviorTreeEditor::CanAddInputPin() const
{
	FGraphPanelSelectionSet CurrentSelection = GetSelectedNodes();
	bool bReturnValue = false;

	// Iterate over all nodes, and make sure all execution sequence nodes will always have at least 2 outs
	for (FGraphPanelSelectionSet::TConstIterator It(CurrentSelection); It; ++It)
	{
		UBehaviorTreeDecoratorGraphNode_Logic* LogicNode = Cast<UBehaviorTreeDecoratorGraphNode_Logic>(*It);
		if (LogicNode)
		{
			bReturnValue = LogicNode->CanAddPins();
			break;
		}
	}

	return bReturnValue;
};

void FBehaviorTreeEditor::OnRemoveInputPin()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = UpdateGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		const FScopedTransaction Transaction( LOCTEXT("RemoveInputPin", "Remove Input Pin") );

		UEdGraphPin* SelectedPin = FocusedGraphEd->GetGraphPinForMenu();
		UEdGraphNode* OwningNode = SelectedPin->GetOwningNode();

		OwningNode->Modify();
		SelectedPin->Modify();

		if (UBehaviorTreeDecoratorGraphNode_Logic* LogicNode = Cast<UBehaviorTreeDecoratorGraphNode_Logic>(OwningNode))
		{
			LogicNode->RemoveInputPin(SelectedPin);
		}

		// Update the graph so that the node will be refreshed
		FocusedGraphEd->NotifyGraphChanged();
	}
}

bool FBehaviorTreeEditor::CanRemoveInputPin() const
{
	FGraphPanelSelectionSet CurrentSelection = GetSelectedNodes();
	bool bReturnValue = false;

	// Iterate over all nodes, and make sure all execution sequence nodes will always have at least 2 outs
	for (FGraphPanelSelectionSet::TConstIterator It(CurrentSelection); It; ++It)
	{
		UBehaviorTreeDecoratorGraphNode_Logic* LogicNode = Cast<UBehaviorTreeDecoratorGraphNode_Logic>(*It);
		if (LogicNode)
		{
			bReturnValue = LogicNode->CanRemovePins();
			break;
		}
	}

	return bReturnValue;
};

void FBehaviorTreeEditor::OnGraphEditorFocused(const TSharedRef<SGraphEditor>& InGraphEditor)
{
	UpdateGraphEdPtr = InGraphEditor;
	FocusedGraphOwner = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(InGraphEditor->GetCurrentGraph()->GetOuter());

	FGraphPanelSelectionSet CurrentSelection;
	CurrentSelection = InGraphEditor->GetSelectedNodes();
	OnSelectedNodesChanged(CurrentSelection);
}

void FBehaviorTreeEditor::SaveAsset_Execute()
{
	if (BehaviorTree)
	{
		UBehaviorTreeGraph* BTGraph = Cast<UBehaviorTreeGraph>(BehaviorTree->BTGraph);
		if (BTGraph)
		{
			BTGraph->OnSave();
		}
	}
	// save it
	IBehaviorTreeEditor::SaveAsset_Execute();
}

void FBehaviorTreeEditor::SetToolbarCreateActionsEnabled(const bool bActionsEnabled)
{
	ToolbarBuilder->SetCreateActionsEnabled(bActionsEnabled);
}

void FBehaviorTreeEditor::OnEnableBreakpoint()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UBehaviorTreeGraphNode* SelectedNode = Cast<UBehaviorTreeGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->bHasBreakpoint && !SelectedNode->bIsBreakpointEnabled)
		{
			SelectedNode->bIsBreakpointEnabled = true;
			Debugger->OnBreakpointAdded(SelectedNode);
		}
	}
}

bool FBehaviorTreeEditor::CanEnableBreakpoint() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UBehaviorTreeGraphNode* SelectedNode = Cast<UBehaviorTreeGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->bHasBreakpoint && !SelectedNode->bIsBreakpointEnabled)
		{
			return true;
		}
	}

	return false;
}

void FBehaviorTreeEditor::OnDisableBreakpoint()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UBehaviorTreeGraphNode* SelectedNode = Cast<UBehaviorTreeGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->bHasBreakpoint && SelectedNode->bIsBreakpointEnabled)
		{
			SelectedNode->bIsBreakpointEnabled = false;
			Debugger->OnBreakpointRemoved(SelectedNode);
		}
	}
}

bool FBehaviorTreeEditor::CanDisableBreakpoint() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UBehaviorTreeGraphNode* SelectedNode = Cast<UBehaviorTreeGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->bHasBreakpoint && SelectedNode->bIsBreakpointEnabled)
		{
			return true;
		}
	}

	return false;
}

void FBehaviorTreeEditor::OnAddBreakpoint()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UBehaviorTreeGraphNode* SelectedNode = Cast<UBehaviorTreeGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->CanPlaceBreakpoints() && !SelectedNode->bHasBreakpoint)
		{
			SelectedNode->bHasBreakpoint = true;
			SelectedNode->bIsBreakpointEnabled = true;
			Debugger->OnBreakpointAdded(SelectedNode);
		}
	}
}

bool FBehaviorTreeEditor::CanAddBreakpoint() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UBehaviorTreeGraphNode* SelectedNode = Cast<UBehaviorTreeGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->CanPlaceBreakpoints() && !SelectedNode->bHasBreakpoint)
		{
			return true;
		}
	}

	return false;
}

void FBehaviorTreeEditor::OnRemoveBreakpoint()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UBehaviorTreeGraphNode* SelectedNode = Cast<UBehaviorTreeGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->CanPlaceBreakpoints() && SelectedNode->bHasBreakpoint)
		{
			SelectedNode->bHasBreakpoint = false;
			SelectedNode->bIsBreakpointEnabled = false;
			Debugger->OnBreakpointRemoved(SelectedNode);
		}
	}
}

bool FBehaviorTreeEditor::CanRemoveBreakpoint() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UBehaviorTreeGraphNode* SelectedNode = Cast<UBehaviorTreeGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->CanPlaceBreakpoints() && SelectedNode->bHasBreakpoint)
		{
			return true;
		}
	}

	return false;
}

void FBehaviorTreeEditor::OnToggleBreakpoint()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UBehaviorTreeGraphNode* SelectedNode = Cast<UBehaviorTreeGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->CanPlaceBreakpoints())
		{
			if (SelectedNode->bHasBreakpoint)
			{
				SelectedNode->bHasBreakpoint = false;
				SelectedNode->bIsBreakpointEnabled = false;
				Debugger->OnBreakpointRemoved(SelectedNode);
			}
			else
			{
				SelectedNode->bHasBreakpoint = true;
				SelectedNode->bIsBreakpointEnabled = true;
				Debugger->OnBreakpointAdded(SelectedNode);
			}
		}
	}
}

bool FBehaviorTreeEditor::CanToggleBreakpoint() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UBehaviorTreeGraphNode* SelectedNode = Cast<UBehaviorTreeGraphNode>(*NodeIt);
		if (SelectedNode && SelectedNode->CanPlaceBreakpoints())
		{
			return true;
		}
	}

	return false;
}

void FBehaviorTreeEditor::JumpToNode(const UEdGraphNode* Node)
{
	TSharedPtr<SDockTab> ActiveTab = DocumentManager->GetActiveTab();
	if (ActiveTab.IsValid())
	{
		TSharedPtr<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(ActiveTab->GetContent());
		if (GraphEditor.IsValid())
		{
			GraphEditor->JumpToNode(Node, false);
		}
	}
}

TWeakPtr<SGraphEditor> FBehaviorTreeEditor::GetFocusedGraphPtr() const
{
	return UpdateGraphEdPtr;
}

bool FBehaviorTreeEditor::CanAccessBlackboardMode() const
{
	return GetBlackboardData() != nullptr;
}

bool FBehaviorTreeEditor::CanAccessBehaviorTreeMode() const
{
	return BehaviorTree != nullptr;
}

FText FBehaviorTreeEditor::GetLocalizedMode(FName InMode)
{
	static TMap< FName, FText > LocModes;

	if (LocModes.Num() == 0)
	{
		LocModes.Add( BehaviorTreeMode, BehaviorTreeModeText );
		LocModes.Add( BlackboardMode, BlackboardModeText );
	}

	check( InMode != NAME_None );
	const FText* OutDesc = LocModes.Find( InMode );
	check( OutDesc );
	return *OutDesc;
}

UEdGraphNode* FBehaviorTreeEditor::FindInjectedNode(int32 Index) const
{
	UBehaviorTreeGraph* BTGraph = Cast<UBehaviorTreeGraph>(BehaviorTree->BTGraph);
	return BTGraph ? BTGraph->FindInjectedNode(Index) : NULL;
}

void FBehaviorTreeEditor::DoubleClickNode(UEdGraphNode* Node)
{
	FocusAttentionOnNode(Node);
	OnNodeDoubleClicked(Node);
}

void FBehaviorTreeEditor::FocusAttentionOnNode(UEdGraphNode* Node)
{
	TSharedPtr<SGraphEditor> CurrentGraphEditor = UpdateGraphEdPtr.Pin();
	if (CurrentGraphEditor.IsValid())
	{
		CurrentGraphEditor->ClearSelectionSet();
		CurrentGraphEditor->SetNodeSelection(Node, true);
	}
	JumpToNode(Node);
}

void FBehaviorTreeEditor::FocusWindow(UObject* ObjectToFocusOn)
{
	if(ObjectToFocusOn == BehaviorTree)
	{
		SetCurrentMode(BehaviorTreeMode);
	}
	else if(ObjectToFocusOn == GetBlackboardData())
	{
		SetCurrentMode(BlackboardMode);
	}

	FWorkflowCentricApplication::FocusWindow(ObjectToFocusOn);
}

bool FBehaviorTreeEditor::IncludeAssetInRestoreOpenAssetsPrompt(UObject* Asset) const
{
	// If we're in a BT editor which has a valid BehaviorTree, then don't reopen the BB during restore, only the BT
	return Asset && (!Asset->IsA(UBlackboardData::StaticClass()) || !BehaviorTree);
}

void FBehaviorTreeEditor::OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
{
	if (NodeBeingChanged)
	{
		static const FText TranscationTitle = FText::FromString(FString(TEXT("Rename Node")));
		const FScopedTransaction Transaction(TranscationTitle);
		NodeBeingChanged->Modify();
		NodeBeingChanged->OnRenameNode(NewText.ToString());
	}
}

void FBehaviorTreeEditor::InitializeDebuggerState(class FBehaviorTreeDebugger* ParentDebugger) const
{
	if (Debugger.IsValid())
	{
		Debugger->InitializeFromParent(ParentDebugger);
	}
}

void FBehaviorTreeEditor::DebuggerSwitchAsset(UBehaviorTree* NewAsset)
{
	if (NewAsset)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewAsset);

		IBehaviorTreeEditor* ChildNodeEditor = static_cast<IBehaviorTreeEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(NewAsset, true));
		if (ChildNodeEditor)
		{
			ChildNodeEditor->InitializeDebuggerState(Debugger.Get());
		}
	}
}

void FBehaviorTreeEditor::DebuggerUpdateGraph()
{
	UBehaviorTreeGraph* BTGraph = BehaviorTree ? Cast<UBehaviorTreeGraph>(BehaviorTree->BTGraph) : NULL;
	if (BTGraph)
	{
		BTGraph->RebuildExecutionOrder();
	}
}

FText FBehaviorTreeEditor::GetToolkitName() const
{
	const UObject* EditingObject = GetCurrentMode() == BehaviorTreeMode ? (UObject*)BehaviorTree : (UObject*)GetBlackboardData();
	if(EditingObject != nullptr)
	{
		return FAssetEditorToolkit::GetLabelForObject(EditingObject);
	}

	return FText();
}

FText FBehaviorTreeEditor::GetToolkitToolTipText() const
{
	const UObject* EditingObject = GetCurrentMode() == BehaviorTreeMode ? (UObject*)BehaviorTree : (UObject*)GetBlackboardData();
	if(EditingObject != nullptr)
	{
		return FAssetEditorToolkit::GetToolTipTextForObject(EditingObject);
	}

	return FText();
}

UBehaviorTree* FBehaviorTreeEditor::GetBehaviorTree() const 
{
	return BehaviorTree; 
}

UBlackboardData* FBehaviorTreeEditor::GetBlackboardData() const 
{
	return BehaviorTree == nullptr ? BlackboardData : ToRawPtr(BehaviorTree->BlackboardAsset);
}

void FBehaviorTreeEditor::RefreshDebugger()
{
	Debugger->Refresh();
}

bool FBehaviorTreeEditor::CanCreateNewTask() const
{
	return !IsDebuggerReady();
}

bool FBehaviorTreeEditor::CanCreateNewDecorator() const
{
	return !IsDebuggerReady();
}

bool FBehaviorTreeEditor::CanCreateNewService() const
{
	return !IsDebuggerReady();
}

namespace FBehaviorTreeEditorUtils {
template <typename Type>
class FNewNodeClassFilter : public IClassViewerFilter
{
public:
	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		if(InClass != nullptr)
		{
			return InClass->IsChildOf(Type::StaticClass());
		}
		return false;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return InUnloadedClassData->IsChildOf(Type::StaticClass());
	}
};
} // namespace FBehaviorTreeEditorUtils


TSharedRef<SWidget> FBehaviorTreeEditor::HandleCreateNewTaskMenu() const
{
	FClassViewerInitializationOptions Options;
	Options.bShowUnloadedBlueprints = true;
	Options.ClassFilters.Add(MakeShareable( new FBehaviorTreeEditorUtils::FNewNodeClassFilter<UBTTask_BlueprintBase> ));

	FOnClassPicked OnPicked( FOnClassPicked::CreateSP( this, &FBehaviorTreeEditor::HandleNewNodeClassPicked ) );

	return FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, OnPicked);
}

TSharedRef<SWidget> FBehaviorTreeEditor::HandleCreateNewDecoratorMenu() const
{
	FClassViewerInitializationOptions Options;
	Options.bShowUnloadedBlueprints = true;
	Options.ClassFilters.Add(MakeShareable( new FBehaviorTreeEditorUtils::FNewNodeClassFilter<UBTDecorator_BlueprintBase> ));

	FOnClassPicked OnPicked( FOnClassPicked::CreateSP( this, &FBehaviorTreeEditor::HandleNewNodeClassPicked ) );

	return FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, OnPicked);
}

TSharedRef<SWidget> FBehaviorTreeEditor::HandleCreateNewServiceMenu() const
{
	FClassViewerInitializationOptions Options;
	Options.bShowUnloadedBlueprints = true;
	Options.ClassFilters.Add(MakeShareable( new FBehaviorTreeEditorUtils::FNewNodeClassFilter<UBTService_BlueprintBase> ));

	FOnClassPicked OnPicked( FOnClassPicked::CreateSP( this, &FBehaviorTreeEditor::HandleNewNodeClassPicked ) );

	return FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, OnPicked);
}
void FBehaviorTreeEditor::HandleNewNodeClassPicked(UClass* InClass) const
{
	UE_CLOGF(InClass == nullptr, LogBehaviorTreeEditor, Error, "Trying to handle new node of NULL class for Behavior Treee %ls "
		, *GetNameSafe(BehaviorTree));

	if(BehaviorTree != nullptr && InClass != nullptr && BehaviorTree->GetOutermost())
	{
		const FString ClassName = FBlueprintEditorUtils::GetClassNameWithoutSuffix(InClass);

		FString PathName = BehaviorTree->GetOutermost()->GetPathName();
		PathName = FPaths::GetPath(PathName);
		
		// Now that we've generated some reasonable default locations/names for the package, allow the user to have the final say
		// before we create the package and initialize the blueprint inside of it.
		FSaveAssetDialogConfig SaveAssetDialogConfig;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
		SaveAssetDialogConfig.DefaultPath = PathName;
		SaveAssetDialogConfig.DefaultAssetName = ClassName + TEXT("_New");
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;

		const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
		if (!SaveObjectPath.IsEmpty())
		{
			const FString SavePackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
			const FString SavePackagePath = FPaths::GetPath(SavePackageName);
			const FString SaveAssetName = FPaths::GetBaseFilename(SavePackageName);

			UPackage* Package = CreatePackage(*SavePackageName);
			if (ensure(Package))
			{
				// Create and init a new Blueprint
				if (UBlueprint* NewBP = FKismetEditorUtilities::CreateBlueprint(InClass, Package, FName(*SaveAssetName), BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass()))
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewBP);

					// Notify the asset registry
					FAssetRegistryModule::AssetCreated(NewBP);

					// Mark the package dirty...
					Package->MarkPackageDirty();
				}
			}
		}
	}

	FSlateApplication::Get().DismissAllMenus();
}

void FBehaviorTreeEditor::CreateNewTask() const
{
	HandleNewNodeClassPicked(UBTTask_BlueprintBase::StaticClass());
}

bool FBehaviorTreeEditor::IsNewTaskButtonVisible() const
{
	return !bHasMultipleTaskBP;
}

bool FBehaviorTreeEditor::IsNewTaskComboVisible() const
{
	return bHasMultipleTaskBP;
}

void FBehaviorTreeEditor::CreateNewDecorator() const
{
	HandleNewNodeClassPicked(UBTDecorator_BlueprintBase::StaticClass());
}

bool FBehaviorTreeEditor::IsNewDecoratorButtonVisible() const
{
	return !bHasMultipleDecoratorBP;
}

bool FBehaviorTreeEditor::IsNewDecoratorComboVisible() const
{
	return bHasMultipleDecoratorBP;
}

void FBehaviorTreeEditor::CreateNewService() const
{
	HandleNewNodeClassPicked(UBTService_BlueprintBase::StaticClass());
}

bool FBehaviorTreeEditor::IsNewServiceButtonVisible() const
{
	return !bHasMultipleServiceBP;
}

bool FBehaviorTreeEditor::IsNewServiceComboVisible() const
{
	return bHasMultipleServiceBP;
}

void FBehaviorTreeEditor::CreateNewBlackboard()
{
	FString PathName = BehaviorTree->GetOutermost()->GetPathName();
	PathName = FPaths::GetPath(PathName);
	const FString PathNameWithFilename = PathName / LOCTEXT("NewBlackboardName", "NewBlackboardData").ToString();

	FString Name;
	FString PackageName;
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(PathNameWithFilename, TEXT(""), PackageName, Name);
	
	UBlackboardDataFactory* Factory = NewObject<UBlackboardDataFactory>();
	UBlackboardData* NewAsset = Cast<UBlackboardData>(AssetToolsModule.Get().CreateAssetWithDialog(Name, PathName, UBlackboardData::StaticClass(), Factory));
	
	if (NewAsset != nullptr)
	{
		UBehaviorTreeGraph* BTGraph = Cast<UBehaviorTreeGraph>(BehaviorTree->BTGraph);
		if (BTGraph)
		{
			// Update root node with the newly created asset
			UBehaviorTreeGraphNode_Root* RootNode = nullptr;
			for (const auto& Node : BTGraph->Nodes)
			{
				RootNode = Cast<UBehaviorTreeGraphNode_Root>(Node);
				if (RootNode != nullptr)
				{
					RootNode->BlackboardAsset = NewAsset;
					break;
				}
			}

			BTGraph->UpdateBlackboardChange();
		}

		UE_CLOGF(BehaviorTree->BlackboardAsset != nullptr, LogBehaviorTreeEditor, Log, "Blackboard data asset %ls has been replaced by %ls", *GetNameSafe(BlackboardData), *GetNameSafe(NewAsset));
		BehaviorTree->BlackboardAsset = NewAsset;
		BlackboardData = NewAsset;

		RefreshBlackboardViewsAssociatedObject();
	}
}

bool FBehaviorTreeEditor::CanCreateNewBlackboard() const
{
	return !IsDebuggerReady();
}

#undef LOCTEXT_NAMESPACE
