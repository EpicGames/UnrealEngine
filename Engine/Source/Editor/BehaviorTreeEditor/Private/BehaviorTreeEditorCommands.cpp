// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeEditorCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

#define LOCTEXT_NAMESPACE "BehaviorTreeEditorCommands"

FBTCommonCommands::FBTCommonCommands() 
	: TCommands<FBTCommonCommands>("BTEditor.Common", LOCTEXT("BehaviorTreeCommandLabel", "Behavior Tree"), NAME_None, FAppStyle::GetAppStyleSetName())
{
}

void FBTCommonCommands::RegisterCommands()
{
	UI_COMMAND(SearchBT, "Search", "Search this Behavior Tree.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::F));
	UI_COMMAND(NewBlackboard, "New Blackboard", "Create a new Blackboard Data Asset", EUserInterfaceActionType::Button, FInputChord());
}

FBTDebuggerCommands::FBTDebuggerCommands() 
	: TCommands<FBTDebuggerCommands>("BTEditor.Debugger", LOCTEXT("Debugger", "Debugger"), NAME_None, FAppStyle::GetAppStyleSetName())
{
}

void FBTDebuggerCommands::RegisterCommands()
{
	UI_COMMAND(BackInto, "Back: Into", "Show state from previous step, can go into subtrees", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(BackOver, "Back: Over", "Show state from previous step, don't go into subtrees", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ForwardInto, "Forward: Into", "Show state from next step, can go into subtrees", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ForwardOver, "Forward: Over", "Show state from next step, don't go into subtrees", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(StepOut, "Step Out", "Show state from next step, leave current subtree", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenParentTree, "Open Parent Tree", "Open the parent tree of this subtree", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenSubtree, "Open Subtree", "Open the currently running subtree", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(PausePlaySession, "Pause", "Pause simulation", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ResumePlaySession, "Resume", "Resume simulation", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND(StopPlaySession, "Stop", "Stop simulation", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(CurrentValues, "Current", "View current values", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SavedValues, "Saved", "View saved values", EUserInterfaceActionType::RadioButton, FInputChord());
}

FBTBlackboardCommands::FBTBlackboardCommands() 
	: TCommands<FBTBlackboardCommands>("BTEditor.Blackboard", LOCTEXT("Blackboard", "Blackboard"), NAME_None, FAppStyle::GetAppStyleSetName())
{
}

void FBTBlackboardCommands::RegisterCommands()
{
	UI_COMMAND(DeleteEntry, "Delete", "Delete this blackboard entry", EUserInterfaceActionType::Button, FInputChord(EKeys::Platform_Delete));
}

#undef LOCTEXT_NAMESPACE
