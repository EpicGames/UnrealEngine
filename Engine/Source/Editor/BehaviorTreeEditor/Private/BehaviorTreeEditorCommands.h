// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Templates/SharedPointer.h"

class FUICommandInfo;

class FBTCommonCommands : public TCommands<FBTCommonCommands>
{
public:
	FBTCommonCommands();

	TSharedPtr<FUICommandInfo> SearchBT;
	TSharedPtr<FUICommandInfo> NewBlackboard;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};

class FBTDebuggerCommands : public TCommands<FBTDebuggerCommands>
{
public:
	FBTDebuggerCommands();

	TSharedPtr<FUICommandInfo> BackInto;
	TSharedPtr<FUICommandInfo> BackOver;
	TSharedPtr<FUICommandInfo> ForwardInto;
	TSharedPtr<FUICommandInfo> ForwardOver;
	TSharedPtr<FUICommandInfo> StepOut;
	TSharedPtr<FUICommandInfo> OpenParentTree;
	TSharedPtr<FUICommandInfo> OpenSubtree;

	TSharedPtr<FUICommandInfo> PausePlaySession;
	TSharedPtr<FUICommandInfo> ResumePlaySession;
	TSharedPtr<FUICommandInfo> StopPlaySession;

	TSharedPtr<FUICommandInfo> CurrentValues;
	TSharedPtr<FUICommandInfo> SavedValues;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};

class FBTBlackboardCommands : public TCommands<FBTBlackboardCommands>
{
public:
	FBTBlackboardCommands();

	TSharedPtr<FUICommandInfo> DeleteEntry;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};
