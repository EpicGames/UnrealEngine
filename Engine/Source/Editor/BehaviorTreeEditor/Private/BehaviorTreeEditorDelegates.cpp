// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeEditorDelegates.h"

namespace UE::BehaviorTreeEditor::Delegates
{
FOnBlackboardKeyChanged OnBlackboardKeyChanged;
}; // UE::BehaviorTreeEditor::Delegates
