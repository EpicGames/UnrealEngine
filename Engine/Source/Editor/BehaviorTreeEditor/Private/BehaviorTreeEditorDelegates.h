// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

struct FBlackboardEntry;
class UBlackboardData;

namespace UE::BehaviorTreeEditor::Delegates
{
/** Delegate for when a blackboard key changes (added, removed, renamed) */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnBlackboardKeyChanged, const UBlackboardData& /*InBlackboardData*/, FBlackboardEntry* const /*InKey*/);
extern FOnBlackboardKeyChanged OnBlackboardKeyChanged;
} // UE::BehaviorTreeEditor::Delegates
