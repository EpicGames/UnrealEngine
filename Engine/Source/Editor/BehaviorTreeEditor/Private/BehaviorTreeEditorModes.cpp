// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeEditorModes.h"

#include "BehaviorTreeEditor.h"
#include "BehaviorTreeEditorTabFactories.h"
#include "BehaviorTreeEditorTabs.h"
#include "BehaviorTreeEditorToolbar.h"
#include "Framework/Docking/TabManager.h"
#include "Misc/AssertionMacros.h"
#include "Types/SlateEnums.h"

/////////////////////////////////////////////////////
// FBehaviorTreeEditorApplicationMode

#define LOCTEXT_NAMESPACE "BehaviorTreeApplicationMode"

FBehaviorTreeEditorApplicationMode::FBehaviorTreeEditorApplicationMode(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditor)
	: FApplicationMode(FBehaviorTreeEditor::BehaviorTreeMode, FBehaviorTreeEditor::GetLocalizedMode)
{
	BehaviorTreeEditor = InBehaviorTreeEditor;

	BehaviorTreeEditorTabFactories.RegisterFactory(MakeShareable(new FBehaviorTreeDetailsSummoner(InBehaviorTreeEditor)));
	BehaviorTreeEditorTabFactories.RegisterFactory(MakeShareable(new FBehaviorTreeSearchSummoner(InBehaviorTreeEditor)));
	BehaviorTreeEditorTabFactories.RegisterFactory(MakeShareable(new FBlackboardSummoner(InBehaviorTreeEditor)));

	TabLayout = FTabManager::NewLayout( "Standalone_BehaviorTree_Layout_v2" )
	->AddArea
	(
		FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter() ->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.7f)
				->AddTab(FBehaviorTreeEditorTabs::GraphEditorID, ETabState::ClosedTab)
			)
			->Split
			(
				FTabManager::NewSplitter() ->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.3f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.6f)
					->AddTab(FBehaviorTreeEditorTabs::GraphDetailsID, ETabState::OpenedTab)
					->AddTab(FBehaviorTreeEditorTabs::SearchID, ETabState::ClosedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.4f)
					->AddTab(FBehaviorTreeEditorTabs::BlackboardID, ETabState::OpenedTab)
				)
			)
		)
	);
	
	InBehaviorTreeEditor->GetToolbarBuilder()->AddModesToolbar(ToolbarExtender);
	InBehaviorTreeEditor->GetToolbarBuilder()->AddDebuggerToolbar(ToolbarExtender);
	InBehaviorTreeEditor->GetToolbarBuilder()->AddBehaviorTreeToolbar(ToolbarExtender);
}

void FBehaviorTreeEditorApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	check(BehaviorTreeEditor.IsValid());
	TSharedPtr<FBehaviorTreeEditor> BehaviorTreeEditorPtr = BehaviorTreeEditor.Pin();
	
	BehaviorTreeEditorPtr->RegisterToolbarTab(InTabManager.ToSharedRef());

	// Mode-specific setup
	BehaviorTreeEditorPtr->PushTabFactories(BehaviorTreeEditorTabFactories);
}

void FBehaviorTreeEditorApplicationMode::PreDeactivateMode()
{
	FApplicationMode::PreDeactivateMode();

	check(BehaviorTreeEditor.IsValid());
	TSharedPtr<FBehaviorTreeEditor> BehaviorTreeEditorPtr = BehaviorTreeEditor.Pin();
	
	BehaviorTreeEditorPtr->SaveEditedObjectState();
}

void FBehaviorTreeEditorApplicationMode::PostActivateMode()
{
	// Reopen any documents that were open when the blueprint was last saved
	check(BehaviorTreeEditor.IsValid());
	TSharedPtr<FBehaviorTreeEditor> BehaviorTreeEditorPtr = BehaviorTreeEditor.Pin();
	BehaviorTreeEditorPtr->RestoreBehaviorTree();

	FApplicationMode::PostActivateMode();
}

#undef  LOCTEXT_NAMESPACE

/////////////////////////////////////////////////////
// FBlackboardEditorApplicationMode

#define LOCTEXT_NAMESPACE "BlackboardApplicationMode"

FBlackboardEditorApplicationMode::FBlackboardEditorApplicationMode(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditor)
	: FApplicationMode(FBehaviorTreeEditor::BlackboardMode, FBehaviorTreeEditor::GetLocalizedMode)
{
	BehaviorTreeEditor = InBehaviorTreeEditor;
	
	BlackboardTabFactories.RegisterFactory(MakeShareable(new FBlackboardEditorSummoner(InBehaviorTreeEditor)));
	BlackboardTabFactories.RegisterFactory(MakeShareable(new FBlackboardDetailsSummoner(InBehaviorTreeEditor)));

	TabLayout = FTabManager::NewLayout( "Standalone_BlackboardEditor_Layout_v2" )
	->AddArea
	(
		FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter() ->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->AddTab(FBehaviorTreeEditorTabs::BlackboardEditorID, ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->AddTab(FBehaviorTreeEditorTabs::BlackboardDetailsID, ETabState::OpenedTab)
			)
		)
	);

	InBehaviorTreeEditor->GetToolbarBuilder()->AddModesToolbar(ToolbarExtender);
}

void FBlackboardEditorApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	check(BehaviorTreeEditor.IsValid());
	TSharedPtr<FBehaviorTreeEditor> BehaviorTreeEditorPtr = BehaviorTreeEditor.Pin();
	
	BehaviorTreeEditorPtr->RegisterToolbarTab(InTabManager.ToSharedRef());

	// Mode-specific setup
	BehaviorTreeEditorPtr->PushTabFactories(BlackboardTabFactories);
}

void FBlackboardEditorApplicationMode::PostActivateMode()
{
	// Reopen any documents that were open when the BT was last saved
	check(BehaviorTreeEditor.IsValid());
	TSharedPtr<FBehaviorTreeEditor> BehaviorTreeEditorPtr = BehaviorTreeEditor.Pin();
//	BehaviorTreeEditorPtr->StartEditingDefaults();

	FApplicationMode::PostActivateMode();
}

#undef LOCTEXT_NAMESPACE
