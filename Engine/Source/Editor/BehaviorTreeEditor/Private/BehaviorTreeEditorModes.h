// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

/** Application mode for main behavior tree editing mode */
class FBehaviorTreeEditorApplicationMode : public FApplicationMode
{
public:
	FBehaviorTreeEditorApplicationMode(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditor);

	virtual void RegisterTabFactories(TSharedPtr<class FTabManager> InTabManager) override;
	virtual void PreDeactivateMode() override;
	virtual void PostActivateMode() override;

protected:
	TWeakPtr<class FBehaviorTreeEditor> BehaviorTreeEditor;

	// Set of spawnable tabs in behavior tree editing mode
	FWorkflowAllowedTabSet BehaviorTreeEditorTabFactories;
};

/** Application mode for blackboard editing mode */
class FBlackboardEditorApplicationMode : public FApplicationMode
{
public:
	FBlackboardEditorApplicationMode(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditor);

	virtual void RegisterTabFactories(TSharedPtr<class FTabManager> InTabManager) override;
	virtual void PostActivateMode() override;

protected:
	TWeakPtr<class FBehaviorTreeEditor> BehaviorTreeEditor;

	// Set of spawnable tabs in blackboard mode
	FWorkflowAllowedTabSet BlackboardTabFactories;
};
