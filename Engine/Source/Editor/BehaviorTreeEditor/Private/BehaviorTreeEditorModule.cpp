// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeEditorModule.h"

#include "DetailCustomizations/BlackboardKeysDetails.h"
#include "BehaviorTree/Decorators/BTDecorator_BlueprintBase.h"
#include "BehaviorTree/Services/BTService_BlueprintBase.h"
#include "BehaviorTree/Tasks/BTTask_BlueprintBase.h"
#include "BehaviorTreeDecoratorGraphNode_Decorator.h"
#include "BehaviorTreeEditor.h"
#include "BehaviorTreeGraphNode.h"
#include "DetailCustomizations/BlackboardDecoratorDetails.h"
#include "DetailCustomizations/BlackboardKeysDetails.h"
#include "DetailCustomizations/BlackboardSelectorDetails.h"
#include "EdGraphUtilities.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SGraphNode_BehaviorTree.h"
#include "SGraphNode_Decorator.h"
#include "ValueOrBBKeyDetails.h"

IMPLEMENT_MODULE(FBehaviorTreeEditorModule, BehaviorTreeEditor);
DEFINE_LOG_CATEGORY(LogBehaviorTreeEditor);

const FName FBehaviorTreeEditorModule::BehaviorTreeEditorAppIdentifier(TEXT("BehaviorTreeEditorApp"));

class FGraphPanelNodeFactory_BehaviorTree : public FGraphPanelNodeFactory
{
	virtual TSharedPtr<class SGraphNode> CreateNode(UEdGraphNode* Node) const override
	{
		if (UBehaviorTreeGraphNode* BTNode = Cast<UBehaviorTreeGraphNode>(Node))
		{
			return SNew(SGraphNode_BehaviorTree, BTNode);
		}

		if (UBehaviorTreeDecoratorGraphNode_Decorator* InnerNode = Cast<UBehaviorTreeDecoratorGraphNode_Decorator>(Node))
		{
			return SNew(SGraphNode_Decorator, InnerNode);
		}

		return nullptr;
	}
};

TSharedPtr<FGraphPanelNodeFactory> GraphPanelNodeFactory_BehaviorTree;

void FBehaviorTreeEditorModule::StartupModule()
{
	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
	ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

	GraphPanelNodeFactory_BehaviorTree = MakeShareable(new FGraphPanelNodeFactory_BehaviorTree());
	FEdGraphUtilities::RegisterVisualNodeFactory(GraphPanelNodeFactory_BehaviorTree);

	// Register the details customizer
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout("BlackboardKeySelector", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FBlackboardSelectorDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("BTDecorator_Blackboard", FOnGetDetailCustomizationInstance::CreateStatic(&FBlackboardDecoratorDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("BTDecorator", FOnGetDetailCustomizationInstance::CreateStatic(&FBehaviorDecoratorDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("BlackboardKeyType_Class", FOnGetDetailCustomizationInstance::CreateStatic(&FBlackboardKeyDetails_Class::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("BlackboardKeyType_Enum", FOnGetDetailCustomizationInstance::CreateStatic(&FBlackboardKeyDetails_Enum::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("BlackboardKeyType_Object", FOnGetDetailCustomizationInstance::CreateStatic(&FBlackboardKeyDetails_Object::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("ValueOrBBKey_Bool", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FValueOrBBKeyDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("ValueOrBBKey_Class", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FValueOrBBKeyDetails_Class::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("ValueOrBBKey_Enum", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FValueOrBBKeyDetails_Enum::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("ValueOrBBKey_Float", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FValueOrBBKeyDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("ValueOrBBKey_Int32", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FValueOrBBKeyDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("ValueOrBBKey_Name", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FValueOrBBKeyDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("ValueOrBBKey_String", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FValueOrBBKeyDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("ValueOrBBKey_Object", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FValueOrBBKeyDetails_Object::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("ValueOrBBKey_Rotator", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FValueOrBBKeyDetails_WithChild::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("ValueOrBBKey_Vector", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FValueOrBBKeyDetails_WithChild::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("ValueOrBBKey_Struct", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FValueOrBBKeyDetails_Struct::MakeInstance));
	PropertyModule.NotifyCustomizationModuleChanged();
}

void FBehaviorTreeEditorModule::ShutdownModule()
{
	if (!UObjectInitialized())
	{
		return;
	}

	MenuExtensibilityManager.Reset();
	ToolBarExtensibilityManager.Reset();
	ClassCache.Reset();

	if (GraphPanelNodeFactory_BehaviorTree.IsValid())
	{
		FEdGraphUtilities::UnregisterVisualNodeFactory(GraphPanelNodeFactory_BehaviorTree);
		GraphPanelNodeFactory_BehaviorTree.Reset();
	}

	// Unregister the details customization
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomPropertyTypeLayout("BlackboardKeySelector");
		PropertyModule.UnregisterCustomClassLayout("BTDecorator_Blackboard");
		PropertyModule.UnregisterCustomClassLayout("BTDecorator");
		PropertyModule.UnregisterCustomClassLayout("BlackboardKeyType_Class");
		PropertyModule.UnregisterCustomClassLayout("BlackboardKeyType_Enum");
		PropertyModule.UnregisterCustomClassLayout("BlackboardKeyType_Object");
		PropertyModule.UnregisterCustomPropertyTypeLayout("ValueOrBBKey_Bool");
		PropertyModule.UnregisterCustomPropertyTypeLayout("ValueOrBBKey_Class");
		PropertyModule.UnregisterCustomPropertyTypeLayout("ValueOrBBKey_Enum");
		PropertyModule.UnregisterCustomPropertyTypeLayout("ValueOrBBKey_Float");
		PropertyModule.UnregisterCustomPropertyTypeLayout("ValueOrBBKey_Int32");
		PropertyModule.UnregisterCustomPropertyTypeLayout("ValueOrBBKey_Name");
		PropertyModule.UnregisterCustomPropertyTypeLayout("ValueOrBBKey_String");
		PropertyModule.UnregisterCustomPropertyTypeLayout("ValueOrBBKey_Object");
		PropertyModule.UnregisterCustomPropertyTypeLayout("ValueOrBBKey_Rotator");
		PropertyModule.UnregisterCustomPropertyTypeLayout("ValueOrBBKey_Vector");
		PropertyModule.UnregisterCustomPropertyTypeLayout("ValueOrBBKey_Struct");
		PropertyModule.NotifyCustomizationModuleChanged();
	}
}

TSharedRef<IBehaviorTreeEditor> FBehaviorTreeEditorModule::CreateBehaviorTreeEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UObject* Object)
{
	if (!ClassCache.IsValid())
	{
		ClassCache = MakeShareable(new FGraphNodeClassHelper(UBTNode::StaticClass()));
		FGraphNodeClassHelper::AddObservedBlueprintClasses(UBTTask_BlueprintBase::StaticClass());
		FGraphNodeClassHelper::AddObservedBlueprintClasses(UBTDecorator_BlueprintBase::StaticClass());
		FGraphNodeClassHelper::AddObservedBlueprintClasses(UBTService_BlueprintBase::StaticClass());
		ClassCache->UpdateAvailableBlueprintClasses();
	}

	TSharedRef<FBehaviorTreeEditor> NewBehaviorTreeEditor(new FBehaviorTreeEditor());
	NewBehaviorTreeEditor->InitBehaviorTreeEditor(Mode, InitToolkitHost, Object);
	return NewBehaviorTreeEditor;
}
