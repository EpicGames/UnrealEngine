// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeEditorTabFactories.h"

#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTreeEditor.h"
#include "BehaviorTreeEditorTabs.h"
#include "Containers/Array.h"
#include "Engine/Blueprint.h"
#include "GraphEditor.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Docking/SDockTab.h"

class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "BehaviorTreeEditorFactories"

FBlackboardSummoner::FBlackboardSummoner(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditorPtr)
	: FWorkflowTabFactory(FBehaviorTreeEditorTabs::BlackboardID, InBehaviorTreeEditorPtr)
	, BehaviorTreeEditorPtr(InBehaviorTreeEditorPtr)
{
	TabLabel = LOCTEXT("BlackboardLabel", "Blackboard");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "BTEditor.Blackboard");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("BlackboardView", "Blackboard");
	ViewMenuTooltip = LOCTEXT("BlackboardView_ToolTip", "Show the blackboard view");
}

TSharedRef<SWidget> FBlackboardSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return BehaviorTreeEditorPtr.Pin()->SpawnBlackboardView();
}

FText FBlackboardSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return LOCTEXT("BlackboardTabTooltip", "The Blackboard view is for viewing and debugging blackboard key/value pairs.");
}

FBlackboardEditorSummoner::FBlackboardEditorSummoner(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditorPtr)
	: FWorkflowTabFactory(FBehaviorTreeEditorTabs::BlackboardEditorID, InBehaviorTreeEditorPtr)
	, BehaviorTreeEditorPtr(InBehaviorTreeEditorPtr)
{
	TabLabel = LOCTEXT("BlackboardLabel", "Blackboard");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "BTEditor.Blackboard");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("BlackboardEditor", "Blackboard");
	ViewMenuTooltip = LOCTEXT("BlackboardEditor_ToolTip", "Show the blackboard editor");
}

TSharedRef<SWidget> FBlackboardEditorSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return BehaviorTreeEditorPtr.Pin()->SpawnBlackboardEditor();
}

FText FBlackboardEditorSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return LOCTEXT("BlackboardEditorTabTooltip", "The Blackboard editor is for editing and debugging blackboard key/value pairs.");
}

FBlackboardDetailsSummoner::FBlackboardDetailsSummoner(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditorPtr)
	: FWorkflowTabFactory(FBehaviorTreeEditorTabs::BlackboardDetailsID, InBehaviorTreeEditorPtr)
	, BehaviorTreeEditorPtr(InBehaviorTreeEditorPtr)
{
	TabLabel = LOCTEXT("BlackboardDetailsLabel", "Blackboard Details");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("BlackboardDetailsView", "Details");
	ViewMenuTooltip = LOCTEXT("BlackboardDetailsView_ToolTip", "Show the details view");
}

TSharedRef<SWidget> FBlackboardDetailsSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	check(BehaviorTreeEditorPtr.IsValid());
	return BehaviorTreeEditorPtr.Pin()->SpawnBlackboardDetails();
}

FText FBlackboardDetailsSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return LOCTEXT("BlackboardDetailsTabTooltip", "The details tab is for editing blackboard entries.");
}

FBehaviorTreeDetailsSummoner::FBehaviorTreeDetailsSummoner(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditorPtr)
	: FWorkflowTabFactory(FBehaviorTreeEditorTabs::GraphDetailsID, InBehaviorTreeEditorPtr)
	, BehaviorTreeEditorPtr(InBehaviorTreeEditorPtr)
{
	TabLabel = LOCTEXT("BehaviorTreeDetailsLabel", "Details");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("BehaviorTreeDetailsView", "Details");
	ViewMenuTooltip = LOCTEXT("BehaviorTreeDetailsView_ToolTip", "Show the details view");
}

TSharedRef<SWidget> FBehaviorTreeDetailsSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	check(BehaviorTreeEditorPtr.IsValid());
	return BehaviorTreeEditorPtr.Pin()->SpawnProperties();
}

FText FBehaviorTreeDetailsSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return LOCTEXT("BehaviorTreeDetailsTabTooltip", "The behavior tree details tab allows editing of the properties of behavior tree nodes");
}

FBehaviorTreeSearchSummoner::FBehaviorTreeSearchSummoner(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditorPtr)
	: FWorkflowTabFactory(FBehaviorTreeEditorTabs::SearchID, InBehaviorTreeEditorPtr)
	, BehaviorTreeEditorPtr(InBehaviorTreeEditorPtr)
{
	TabLabel = LOCTEXT("BehaviorTreeSearchLabel", "Search");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.FindResults");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("BehaviorTreeSearchView", "Search");
	ViewMenuTooltip = LOCTEXT("BehaviorTreeSearchView_ToolTip", "Show the behavior tree search tab");
}

TSharedRef<SWidget> FBehaviorTreeSearchSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return BehaviorTreeEditorPtr.Pin()->SpawnSearch();
}

FText FBehaviorTreeSearchSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return LOCTEXT("BehaviorTreeSearchTabTooltip", "The behavior tree search tab allows searching within behavior tree nodes");
}

FBTGraphEditorSummoner::FBTGraphEditorSummoner(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditorPtr, FOnCreateGraphEditorWidget CreateGraphEditorWidgetCallback)
	: FDocumentTabFactoryForObjects<UEdGraph>(FBehaviorTreeEditorTabs::GraphEditorID, InBehaviorTreeEditorPtr)
	, BehaviorTreeEditorPtr(InBehaviorTreeEditorPtr)
	, OnCreateGraphEditorWidget(CreateGraphEditorWidgetCallback)
{
}

void FBTGraphEditorSummoner::OnTabActivated(TSharedPtr<SDockTab> Tab) const
{
	check(BehaviorTreeEditorPtr.IsValid());
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
	BehaviorTreeEditorPtr.Pin()->OnGraphEditorFocused(GraphEditor);
}

void FBTGraphEditorSummoner::OnTabRefreshed(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
	GraphEditor->NotifyGraphChanged();
}

TAttribute<FText> FBTGraphEditorSummoner::ConstructTabNameForObject(UEdGraph* DocumentID) const
{
	return TAttribute<FText>( FText::FromString( DocumentID->GetName() ) );
}

TSharedRef<SWidget> FBTGraphEditorSummoner::CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const
{
	return OnCreateGraphEditorWidget.Execute(DocumentID);
}

const FSlateBrush* FBTGraphEditorSummoner::GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const
{
	return FAppStyle::Get().GetBrush("BTEditor.BehaviorTree");
}

void FBTGraphEditorSummoner::SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const
{
	check(BehaviorTreeEditorPtr.IsValid());
	check(BehaviorTreeEditorPtr.Pin()->GetBehaviorTree());

	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());

	FVector2f ViewLocation;
	float ZoomAmount;
	GraphEditor->GetViewLocation(ViewLocation, ZoomAmount);

	UEdGraph* Graph = FTabPayload_UObject::CastChecked<UEdGraph>(Payload);
	BehaviorTreeEditorPtr.Pin()->GetBehaviorTree()->LastEditedDocuments.Add(FEditedDocumentInfo(Graph, ViewLocation, ZoomAmount));
}

#undef LOCTEXT_NAMESPACE
