// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"

class SDockTab;
class SGraphEditor;
class SWidget;
struct FSlateBrush;

struct FBlackboardSummoner : public FWorkflowTabFactory
{
public:
	FBlackboardSummoner(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditorPtr);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<class FBehaviorTreeEditor> BehaviorTreeEditorPtr;
};

struct FBlackboardEditorSummoner : public FWorkflowTabFactory
{
public:
	FBlackboardEditorSummoner(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditorPtr);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<class FBehaviorTreeEditor> BehaviorTreeEditorPtr;
};

struct FBlackboardDetailsSummoner : public FWorkflowTabFactory
{
public:
	FBlackboardDetailsSummoner(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditorPtr);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<class FBehaviorTreeEditor> BehaviorTreeEditorPtr;
};

struct FBehaviorTreeDetailsSummoner : public FWorkflowTabFactory
{
public:
	FBehaviorTreeDetailsSummoner(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditorPtr);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<class FBehaviorTreeEditor> BehaviorTreeEditorPtr;
};

struct FBehaviorTreeSearchSummoner : public FWorkflowTabFactory
{
public:
	FBehaviorTreeSearchSummoner(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditorPtr);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<class FBehaviorTreeEditor> BehaviorTreeEditorPtr;
};

struct FBTGraphEditorSummoner : public FDocumentTabFactoryForObjects<UEdGraph>
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SGraphEditor>, FOnCreateGraphEditorWidget, UEdGraph*);
public:
	FBTGraphEditorSummoner(TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditorPtr, FOnCreateGraphEditorWidget CreateGraphEditorWidgetCallback);

	virtual void OnTabActivated(TSharedPtr<SDockTab> Tab) const override;
	virtual void OnTabRefreshed(TSharedPtr<SDockTab> Tab) const override;

protected:
	virtual TAttribute<FText> ConstructTabNameForObject(UEdGraph* DocumentID) const override;
	virtual TSharedRef<SWidget> CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const override;
	virtual const FSlateBrush* GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const override;
	virtual void SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const override;

protected:
	TWeakPtr<class FBehaviorTreeEditor> BehaviorTreeEditorPtr;
	FOnCreateGraphEditorWidget OnCreateGraphEditorWidget;
};
