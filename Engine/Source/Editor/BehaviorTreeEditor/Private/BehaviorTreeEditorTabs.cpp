// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeEditorTabs.h"

#include "HAL/Platform.h"
#include "UObject/NameTypes.h"

// Tab identifiers
const FName FBehaviorTreeEditorTabs::GraphDetailsID(TEXT("BehaviorTreeEditor_Properties"));
const FName FBehaviorTreeEditorTabs::SearchID(TEXT("BehaviorTreeEditor_Search"));
const FName FBehaviorTreeEditorTabs::BlackboardID(TEXT("BehaviorTreeEditor_Blackboard"));
const FName FBehaviorTreeEditorTabs::BlackboardEditorID(TEXT("BehaviorTreeEditor_BlackboardEditor"));
const FName FBehaviorTreeEditorTabs::BlackboardDetailsID(TEXT("BehaviorTreeEditor_BlackboardProperties"));

// Document tab identifiers
const FName FBehaviorTreeEditorTabs::GraphEditorID(TEXT("Document"));
