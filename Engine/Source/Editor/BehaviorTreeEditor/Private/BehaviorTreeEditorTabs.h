// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FName;

struct FBehaviorTreeEditorTabs
{
	// Tab identifiers
	static const FName GraphDetailsID;
	static const FName SearchID;
	static const FName BlackboardID;
	static const FName BlackboardEditorID;
	static const FName BlackboardDetailsID;

	// Document tab identifiers
	static const FName GraphEditorID;
};

