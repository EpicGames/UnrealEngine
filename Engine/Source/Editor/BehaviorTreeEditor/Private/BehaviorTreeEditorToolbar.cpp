// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeEditorToolbar.h"

#include "BehaviorTreeEditor.h"
#include "BehaviorTreeEditorCommands.h"
#include "Delegates/Delegate.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/SlateDelegates.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "WorkflowOrientedApp/SModeWidget.h"

class SWidget;

#define LOCTEXT_NAMESPACE "BehaviorTreeEditorToolbar"

class SBehaviorTreeModeSeparator : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SBehaviorTreeModeSeparator) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArg)
	{
		SBorder::Construct(
			SBorder::FArguments()
			.BorderImage(FAppStyle::GetBrush("BlueprintEditor.PipelineSeparator"))
			.Padding(0.0f)
			);
	}

	// SWidget interface
	virtual FVector2D ComputeDesiredSize(float) const override
	{
		const float Height = 20.0f;
		const float Thickness = 16.0f;
		return FVector2D(Thickness, Height);
	}
	// End of SWidget interface
};

void FBehaviorTreeEditorToolbar::AddModesToolbar(TSharedPtr<FExtender> Extender)
{
	check(BehaviorTreeEditor.IsValid());
	TSharedPtr<FBehaviorTreeEditor> BehaviorTreeEditorPtr = BehaviorTreeEditor.Pin();

	Extender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		BehaviorTreeEditorPtr->GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateSP( this, &FBehaviorTreeEditorToolbar::FillModesToolbar ) );
}

void FBehaviorTreeEditorToolbar::AddDebuggerToolbar(TSharedPtr<FExtender> Extender)
{
	// setup toolbar
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, TWeakPtr<FBehaviorTreeEditor> BehaviorTreeEditor)
		{
			TSharedPtr<FBehaviorTreeEditor> BehaviorTreeEditorPtr = BehaviorTreeEditor.Pin();

			const bool bCanShowDebugger = BehaviorTreeEditorPtr->IsDebuggerReady();
			if (bCanShowDebugger)
			{
				TSharedRef<SWidget> SelectionBox = SNew(SComboButton)
					.OnGetMenuContent( BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::OnGetDebuggerActorsMenu )
					.ButtonContent()
					[
						SNew(STextBlock)
						.ToolTipText( LOCTEXT("SelectDebugActor", "Pick actor to debug") )
						.Text(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::GetDebuggerActorDesc )
					];

				ToolbarBuilder.BeginSection("CachedState");
				{
					ToolbarBuilder.AddToolBarButton(FBTDebuggerCommands::Get().BackOver);
					ToolbarBuilder.AddToolBarButton(FBTDebuggerCommands::Get().BackInto);
					ToolbarBuilder.AddToolBarButton(FBTDebuggerCommands::Get().ForwardInto);
					ToolbarBuilder.AddToolBarButton(FBTDebuggerCommands::Get().ForwardOver);
					ToolbarBuilder.AddToolBarButton(FBTDebuggerCommands::Get().StepOut);
					ToolbarBuilder.AddToolBarButton(FBTDebuggerCommands::Get().OpenParentTree);
					ToolbarBuilder.AddToolBarButton(FBTDebuggerCommands::Get().OpenSubtree);
				}
				ToolbarBuilder.EndSection();
				ToolbarBuilder.BeginSection("World");
				{
					ToolbarBuilder.AddToolBarButton(FBTDebuggerCommands::Get().PausePlaySession);
					ToolbarBuilder.AddToolBarButton(FBTDebuggerCommands::Get().ResumePlaySession);
					ToolbarBuilder.AddToolBarButton(FBTDebuggerCommands::Get().StopPlaySession);
					ToolbarBuilder.AddSeparator();
					ToolbarBuilder.AddWidget(SelectionBox);
				}
				ToolbarBuilder.EndSection();
			}
		}
	};

	TSharedPtr<FBehaviorTreeEditor> BehaviorTreeEditorPtr = BehaviorTreeEditor.Pin();

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	ToolbarExtender->AddToolBarExtension("Asset", EExtensionHook::After, BehaviorTreeEditorPtr->GetToolkitCommands(), FToolBarExtensionDelegate::CreateStatic( &Local::FillToolbar, BehaviorTreeEditor ));
	BehaviorTreeEditorPtr->AddToolbarExtender(ToolbarExtender);
}


void FBehaviorTreeEditorToolbar::AddBehaviorTreeToolbar(TSharedPtr<FExtender> Extender)
{
	check(BehaviorTreeEditor.IsValid());
	TSharedPtr<FBehaviorTreeEditor> BehaviorTreeEditorPtr = BehaviorTreeEditor.Pin();

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	ToolbarExtender->AddToolBarExtension("Asset", EExtensionHook::After, BehaviorTreeEditorPtr->GetToolkitCommands(), FToolBarExtensionDelegate::CreateSP( this, &FBehaviorTreeEditorToolbar::FillBehaviorTreeToolbar ));
	BehaviorTreeEditorPtr->AddToolbarExtender(ToolbarExtender);
}

void FBehaviorTreeEditorToolbar::SetCreateActionsEnabled(const bool bActionsEnabled)
{
	bCreateActionsEnabled = bActionsEnabled;
}

void FBehaviorTreeEditorToolbar::FillModesToolbar(FToolBarBuilder& ToolbarBuilder)
{
	check(BehaviorTreeEditor.IsValid());
	TSharedPtr<FBehaviorTreeEditor> BehaviorTreeEditorPtr = BehaviorTreeEditor.Pin();

	TAttribute<FName> GetActiveMode(BehaviorTreeEditorPtr.ToSharedRef(), &FBehaviorTreeEditor::GetCurrentMode);
	FOnModeChangeRequested SetActiveMode = FOnModeChangeRequested::CreateSP(BehaviorTreeEditorPtr.ToSharedRef(), &FBehaviorTreeEditor::SetCurrentMode);

	// Left side padding
	BehaviorTreeEditorPtr->AddToolbarWidget(SNew(SSpacer).Size(FVector2D(4.0f, 1.0f)));

	BehaviorTreeEditorPtr->AddToolbarWidget(
		SNew(SModeWidget, FBehaviorTreeEditor::GetLocalizedMode( FBehaviorTreeEditor::BehaviorTreeMode ), FBehaviorTreeEditor::BehaviorTreeMode)
		.OnGetActiveMode(GetActiveMode)
		.OnSetActiveMode(SetActiveMode)
		.CanBeSelected(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::CanAccessBehaviorTreeMode)
		.ToolTipText(LOCTEXT("BehaviorTreeModeButtonTooltip", "Switch to Behavior Tree Mode"))
		.IconImage(FAppStyle::GetBrush("BTEditor.SwitchToBehaviorTreeMode"))
	);

	BehaviorTreeEditorPtr->AddToolbarWidget(SNew(SSpacer).Size(FVector2D(10.0f, 1.0f)));

	BehaviorTreeEditorPtr->AddToolbarWidget(
		SNew(SModeWidget, FBehaviorTreeEditor::GetLocalizedMode( FBehaviorTreeEditor::BlackboardMode ), FBehaviorTreeEditor::BlackboardMode)
		.OnGetActiveMode(GetActiveMode)
		.OnSetActiveMode(SetActiveMode)
		.CanBeSelected(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::CanAccessBlackboardMode)
		.ToolTipText(LOCTEXT("BlackboardModeButtonTooltip", "Switch to Blackboard Mode"))
		.IconImage(FAppStyle::GetBrush("BTEditor.SwitchToBlackboardMode"))
	);
		
	// Right side padding
	BehaviorTreeEditorPtr->AddToolbarWidget(SNew(SSpacer).Size(FVector2D(10.0f, 1.0f)));
}

void FBehaviorTreeEditorToolbar::FillBehaviorTreeToolbar(FToolBarBuilder& ToolbarBuilder)
{
	if (!bCreateActionsEnabled)
	{
		return;
	}

	check(BehaviorTreeEditor.IsValid());
	TSharedPtr<FBehaviorTreeEditor> BehaviorTreeEditorPtr = BehaviorTreeEditor.Pin();

	if (!BehaviorTreeEditorPtr->IsDebuggerReady() && BehaviorTreeEditorPtr->GetCurrentMode() == FBehaviorTreeEditor::BehaviorTreeMode)
	{
		ToolbarBuilder.BeginSection("Blackboard");
		{
			ToolbarBuilder.AddToolBarButton(FBTCommonCommands::Get().NewBlackboard);
		}
		ToolbarBuilder.EndSection();

		ToolbarBuilder.BeginSection("BehaviorTree");
		{
			const FText NewTaskLabel = LOCTEXT( "NewTask_Label", "New Task" );
			const FText NewTaskTooltip = LOCTEXT( "NewTask_ToolTip", "Create a new task node Blueprint from a base class" );
			const FSlateIcon NewTaskIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "BTEditor.Graph.NewTask");

			ToolbarBuilder.AddToolBarButton(
				FUIAction(
					FExecuteAction::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::CreateNewTask),
					FCanExecuteAction::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::CanCreateNewTask),
					FIsActionChecked(),
					FIsActionButtonVisible::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::IsNewTaskButtonVisible)
					), 
				NAME_None,
				NewTaskLabel,
				NewTaskTooltip,
				NewTaskIcon
			);

			ToolbarBuilder.AddComboButton(
				FUIAction(
					FExecuteAction(),
					FCanExecuteAction::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::CanCreateNewTask),
					FIsActionChecked(),
					FIsActionButtonVisible::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::IsNewTaskComboVisible)
					), 
				FOnGetContent::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::HandleCreateNewTaskMenu),
				NewTaskLabel,
				NewTaskTooltip,
				NewTaskIcon
			);

			const FText NewDecoratorLabel = LOCTEXT( "NewDecorator_Label", "New Decorator" );
			const FText NewDecoratorTooltip = LOCTEXT( "NewDecorator_ToolTip", "Create a new decorator node Blueprint from a base class" );
			const FSlateIcon NewDecoratorIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "BTEditor.Graph.NewDecorator");

			ToolbarBuilder.AddToolBarButton(
				FUIAction(
					FExecuteAction::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::CreateNewDecorator),
					FCanExecuteAction::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::CanCreateNewDecorator),
					FIsActionChecked(),
					FIsActionButtonVisible::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::IsNewDecoratorButtonVisible)
					), 
				NAME_None,
				NewDecoratorLabel,
				NewDecoratorTooltip,
				NewDecoratorIcon
			);

			ToolbarBuilder.AddComboButton(
				FUIAction(
					FExecuteAction(),
					FCanExecuteAction::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::CanCreateNewDecorator),
					FIsActionChecked(),
					FIsActionButtonVisible::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::IsNewDecoratorComboVisible)
					), 
				FOnGetContent::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::HandleCreateNewDecoratorMenu),
				NewDecoratorLabel,
				NewDecoratorTooltip,
				NewDecoratorIcon
			);

			const FText NewServiceLabel = LOCTEXT( "NewService_Label", "New Service" );
			const FText NewServiceTooltip = LOCTEXT( "NewService_ToolTip", "Create a new service node Blueprint from a base class" );
			const FSlateIcon NewServiceIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "BTEditor.Graph.NewService");

			ToolbarBuilder.AddToolBarButton(
				FUIAction(
					FExecuteAction::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::CreateNewService),
					FCanExecuteAction::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::CanCreateNewService),
					FIsActionChecked(),
					FIsActionButtonVisible::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::IsNewServiceButtonVisible)
					), 
				NAME_None,
				NewServiceLabel,
				NewServiceTooltip,
				NewServiceIcon
			);

			ToolbarBuilder.AddComboButton(
				FUIAction(
					FExecuteAction(),
					FCanExecuteAction::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::CanCreateNewService),
					FIsActionChecked(),
					FIsActionButtonVisible::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::IsNewServiceComboVisible)
					), 
				FOnGetContent::CreateSP(BehaviorTreeEditorPtr.Get(), &FBehaviorTreeEditor::HandleCreateNewServiceMenu),
				NewServiceLabel,
				NewServiceTooltip,
				NewServiceIcon
			);
		}
		ToolbarBuilder.EndSection();
	}
}

#undef LOCTEXT_NAMESPACE
