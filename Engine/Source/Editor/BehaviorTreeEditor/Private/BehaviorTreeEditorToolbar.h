// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FBehaviorTreeEditor;
class FExtender;
class FToolBarBuilder;

class FBehaviorTreeEditorToolbar : public TSharedFromThis<FBehaviorTreeEditorToolbar>
{
public:
	FBehaviorTreeEditorToolbar(TSharedPtr<FBehaviorTreeEditor> InBehaviorTreeEditor)
		: BehaviorTreeEditor(InBehaviorTreeEditor) {}

	void AddModesToolbar(TSharedPtr<FExtender> Extender);
	void AddDebuggerToolbar(TSharedPtr<FExtender> Extender);
	void AddBehaviorTreeToolbar(TSharedPtr<FExtender> Extender);

	void SetCreateActionsEnabled(bool bActionsEnabled);

private:
	void FillModesToolbar(FToolBarBuilder& ToolbarBuilder);
	void FillDebuggerToolbar(FToolBarBuilder& ToolbarBuilder);
	void FillBehaviorTreeToolbar(FToolBarBuilder& ToolbarBuilder);

protected:
	/** Pointer back to the blueprint editor tool that owns us */
	TWeakPtr<FBehaviorTreeEditor> BehaviorTreeEditor;

private:
	bool bCreateActionsEnabled = true;
};
