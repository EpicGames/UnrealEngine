// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeEditorTypes.h"

#include "UObject/NameTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeEditorTypes)

const FName UBehaviorTreeEditorTypes::PinCategory_MultipleNodes("MultipleNodes");
const FName UBehaviorTreeEditorTypes::PinCategory_SingleComposite("SingleComposite");
const FName UBehaviorTreeEditorTypes::PinCategory_SingleTask("SingleTask");
const FName UBehaviorTreeEditorTypes::PinCategory_SingleNode("SingleNode");

UBehaviorTreeEditorTypes::UBehaviorTreeEditorTypes(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}
