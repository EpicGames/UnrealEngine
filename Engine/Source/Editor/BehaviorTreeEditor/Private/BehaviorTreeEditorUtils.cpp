// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeEditorUtils.h"

#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTreeDecoratorGraphNode_Decorator.h"
#include "BehaviorTreeGraphNode_Composite.h"
#include "BehaviorTreeGraphNode_CompositeDecorator.h"
#include "BehaviorTreeGraphNode_Decorator.h"
#include "BehaviorTreeGraphNode_Service.h"
#include "BehaviorTreeGraphNode_Task.h"
#include "EdGraph/EdGraph.h"
#include "HAL/PlatformCrt.h"
#include "Templates/Casts.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

TArray<UObject*> BehaviorTreeEditorUtils::GetSelectionForPropertyEditor(const TSet<UObject*>& InSelection, FPropertySelectionInfo& OutSelectionInfo)
{
	TArray<UObject*> Selection;

	for (UObject* SelectionEntry : InSelection)
	{
		UBehaviorTreeGraphNode_Composite* GraphNode_Composite = Cast<UBehaviorTreeGraphNode_Composite>(SelectionEntry);
		if (GraphNode_Composite)
		{
			Selection.Add(GraphNode_Composite->NodeInstance);
			continue;
		}

		UBehaviorTreeGraphNode_Task* GraphNode_Task = Cast<UBehaviorTreeGraphNode_Task>(SelectionEntry);
		if (GraphNode_Task)
		{
			Selection.Add(GraphNode_Task->NodeInstance);
			continue;
		}

		UBehaviorTreeGraphNode_Decorator* GraphNode_Decorator1 = Cast<UBehaviorTreeGraphNode_Decorator>(SelectionEntry);
		if (GraphNode_Decorator1)
		{
			Selection.Add(GraphNode_Decorator1->NodeInstance);
			OutSelectionInfo.FoundDecorator = Cast<UBTDecorator>(GraphNode_Decorator1->NodeInstance);
			OutSelectionInfo.bInjectedNode = OutSelectionInfo.bInjectedNode || GraphNode_Decorator1->bInjectedNode;
			OutSelectionInfo.bRootLevelNode = GraphNode_Decorator1->bRootLevel;
			continue;
		}

		UBehaviorTreeDecoratorGraphNode_Decorator* GraphNode_Decorator2 = Cast<UBehaviorTreeDecoratorGraphNode_Decorator>(SelectionEntry);
		if (GraphNode_Decorator2)
		{
			Selection.Add(GraphNode_Decorator2->NodeInstance);
			OutSelectionInfo.bInjectedNode = OutSelectionInfo.bInjectedNode || !GraphNode_Decorator2->GetGraph()->bEditable;
			continue;
		}
			
		UBehaviorTreeGraphNode_Service* GraphNode_Service = Cast<UBehaviorTreeGraphNode_Service>(SelectionEntry);
		if (GraphNode_Service)
		{
			Selection.Add(GraphNode_Service->NodeInstance);
			continue;
		}

		UBehaviorTreeGraphNode_CompositeDecorator* GraphNode_CompDecorator = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(SelectionEntry);
		if (GraphNode_CompDecorator)
		{
			OutSelectionInfo.FoundGraphNode_CompDecorator = GraphNode_CompDecorator;
			OutSelectionInfo.bInjectedNode = OutSelectionInfo.bInjectedNode || GraphNode_CompDecorator->bInjectedNode;
			OutSelectionInfo.bRootLevelNode = OutSelectionInfo.bRootLevelNode || GraphNode_CompDecorator->bRootLevel;
		}

		Selection.Add(SelectionEntry);
	}

	return Selection;
}
