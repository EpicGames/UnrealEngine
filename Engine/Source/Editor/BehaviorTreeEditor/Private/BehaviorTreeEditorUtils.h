// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"

class UBTDecorator;
class UBehaviorTreeGraphNode_CompositeDecorator;
class UObject;

namespace BehaviorTreeEditorUtils
{
	struct FPropertySelectionInfo
	{
		FPropertySelectionInfo()
			: FoundGraphNode_CompDecorator(nullptr)
			, FoundDecorator(nullptr)
			, bInjectedNode(false)
			, bRootLevelNode(false)
		{
		}

		UBehaviorTreeGraphNode_CompositeDecorator* FoundGraphNode_CompDecorator;
		UBTDecorator* FoundDecorator;
		bool bInjectedNode;
		bool bRootLevelNode;
	};

	/** Given a selection of nodes, return the instances that should be selected be selected for editing in the property panel */
	TArray<UObject*> GetSelectionForPropertyEditor(const TSet<UObject*>& InSelection, FPropertySelectionInfo& OutSelectionInfo);
}
