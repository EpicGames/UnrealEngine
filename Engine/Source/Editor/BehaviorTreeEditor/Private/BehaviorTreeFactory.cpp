// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeFactory.h"

#include "BehaviorTree/BehaviorTree.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeFactory)

class FFeedbackContext;
class UObject;

#define LOCTEXT_NAMESPACE "BehaviorTreeFactory"

UBehaviorTreeFactory::UBehaviorTreeFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UBehaviorTree::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

bool UBehaviorTreeFactory::CanCreateNew() const
{
	return true;
}

UObject* UBehaviorTreeFactory::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UBehaviorTree::StaticClass()));
	return NewObject<UBehaviorTree>(InParent, Class, Name, Flags);;
}

#undef LOCTEXT_NAMESPACE
