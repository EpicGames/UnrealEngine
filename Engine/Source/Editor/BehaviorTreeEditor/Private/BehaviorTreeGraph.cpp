// Copyright Epic Games, Inc. All Rights Reserved.
#include "BehaviorTreeGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "SGraphNode.h"
#include "BehaviorTree/BTNode.h"
#include "BehaviorTree/BTService.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "AIGraphTypes.h"
#include "BehaviorTreeEditorTypes.h"
#include "BehaviorTreeGraphNode.h"
#include "BehaviorTreeGraphNode_Composite.h"
#include "BehaviorTreeGraphNode_CompositeDecorator.h"
#include "BehaviorTreeGraphNode_Decorator.h"
#include "BehaviorTreeGraphNode_Root.h"
#include "BehaviorTreeGraphNode_Service.h"
#include "BehaviorTreeGraphNode_Task.h"
#include "EdGraphSchema_BehaviorTree.h"
#include "SGraphPanel.h"
#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTree/Tasks/BTTask_RunBehavior.h"
#include "BehaviorTree/Composites/BTComposite_SimpleParallel.h"
#include "BehaviorTree/Tasks/BTTask_Wait.h"
#include "BehaviorTreeGraphNode_SimpleParallel.h"
#include "BehaviorTreeGraphNode_SubtreeTask.h"
#include "BehaviorTreeDecoratorGraphNode_Decorator.h"

//////////////////////////////////////////////////////////////////////////
// BehaviorTreeGraph

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeGraph)

namespace BTGraphVersion
{
	const int32 Initial = 0;
	const int32 UnifiedSubNodes = 1;
	const int32 InnerGraphWhitespace = 2;
	const int32 RunBehaviorInSeparateGraph = 3;

	const int32 Latest = RunBehaviorInSeparateGraph;
}

UBehaviorTreeGraph::UBehaviorTreeGraph(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	Schema = UEdGraphSchema_BehaviorTree::StaticClass();
	RootNodeClass = UBehaviorTreeGraphNode_Root::StaticClass();
}

void UBehaviorTreeGraph::UpdateBlackboardChange()
{
	UBehaviorTree* BTAsset = Cast<UBehaviorTree>(GetOuter());
	if (BTAsset == nullptr)
	{
		return;
	}

	for (int32 Index = 0; Index < Nodes.Num(); ++Index)
	{
		UBehaviorTreeGraphNode* MyNode = Cast<UBehaviorTreeGraphNode>(Nodes[Index]);

		if (MyNode)
		{
			UBTNode* MyNodeInstance = Cast<UBTNode>(MyNode->NodeInstance);
			if (MyNodeInstance)
			{
				FScopedBTLoggingContext LogContext(MyNodeInstance);
				MyNodeInstance->InitializeFromAsset(*BTAsset);
			}

			for (int32 iDecorator = 0; iDecorator < MyNode->Decorators.Num(); iDecorator++)
			{
				UBTNode* DecoratorNodeInstance = MyNode->Decorators[iDecorator] ? Cast<UBTNode>(MyNode->Decorators[iDecorator]->NodeInstance) : NULL;
				if (DecoratorNodeInstance)
				{
					FScopedBTLoggingContext LogContext(DecoratorNodeInstance);
					DecoratorNodeInstance->InitializeFromAsset(*BTAsset);
				}

				UBehaviorTreeGraphNode_CompositeDecorator* CompDecoratorNode = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(MyNode->Decorators[iDecorator]);
				if (CompDecoratorNode)
				{
					CompDecoratorNode->OnBlackboardUpdate();
				}
			}

			for (int32 iService = 0; iService < MyNode->Services.Num(); iService++)
			{
				UBTNode* ServiceNodeInstance = MyNode->Services[iService] ? Cast<UBTNode>(MyNode->Services[iService]->NodeInstance) : NULL;
				if (ServiceNodeInstance)
				{
					FScopedBTLoggingContext LogContext(ServiceNodeInstance);
					ServiceNodeInstance->InitializeFromAsset(*BTAsset);
				}
			}
		}
	}

	UpdateErrorMessages();
}

void UBehaviorTreeGraph::UpdateAsset(int32 UpdateFlags)
{
	if (bLockUpdates)
	{
		return;
	}

	// initial cleanup & root node search
	UBehaviorTreeGraphNode_Root* RootNode = NULL;
	for (int32 Index = 0; Index < Nodes.Num(); ++Index)
	{
		UBehaviorTreeGraphNode* Node = Cast<UBehaviorTreeGraphNode>(Nodes[Index]);

		if (Node == nullptr)
		{
			// ignore non-BT nodes.
			continue;
		}

		// debugger flags
		if (UpdateFlags & ClearDebuggerFlags)
		{
			Node->ClearDebuggerState();

			for (int32 iAux = 0; iAux < Node->Services.Num(); iAux++)
			{
				Node->Services[iAux]->ClearDebuggerState();
			}

			for (int32 iAux = 0; iAux < Node->Decorators.Num(); iAux++)
			{
				Node->Decorators[iAux]->ClearDebuggerState();
			}
		}

		// parent chain
		Node->ParentNode = NULL;
		for (int32 iAux = 0; iAux < Node->Services.Num(); iAux++)
		{
			Node->Services[iAux]->ParentNode = Node;
		}

		for (int32 iAux = 0; iAux < Node->Decorators.Num(); iAux++)
		{
			Node->Decorators[iAux]->ParentNode = Node;
		}

		// prepare node instance
		UBTNode* NodeInstance = Cast<UBTNode>(Node->NodeInstance);
		if (NodeInstance)
		{
			// mark all nodes as disconnected first, path from root will replace it with valid values later
			NodeInstance->InitializeNode(NULL, MAX_uint16, 0, 0);
		}

		// cache root
		if (RootNode == NULL)
		{
			RootNode = Cast<UBehaviorTreeGraphNode_Root>(Nodes[Index]);
		}

		UBehaviorTreeGraphNode_CompositeDecorator* CompositeDecorator = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(Nodes[Index]);
		if (CompositeDecorator)
		{
			CompositeDecorator->ResetExecutionRange();
		}
	}

	// we can't look at pins until pin references have been fixed up post undo:
	UEdGraphPin::ResolveAllPinReferences();

	UBehaviorTreeGraphNode* Node = nullptr;
	if (RootNode && RootNode->Pins.Num() > 0 && RootNode->Pins[0]->LinkedTo.Num() > 0)
	{
		Node = Cast<UBehaviorTreeGraphNode>(RootNode->Pins[0]->LinkedTo[0]->GetOwningNode());
	}

	CreateBTFromGraph(Node);

	if ((UpdateFlags & KeepRebuildCounter) == 0)
	{
		ModCounter++;
	}

	UpdateBlackboardChange();
}

void UBehaviorTreeGraph::OnCreated()
{
	Super::OnCreated();
	SpawnMissingNodes();
}

void UBehaviorTreeGraph::OnLoaded()
{
	Super::OnLoaded();
	UpdatePinConnectionTypes();
	RemoveUnknownSubNodes();
}

void UBehaviorTreeGraph::Initialize()
{
	Super::Initialize();
	UpdateInjectedNodes();
}

void UBehaviorTreeGraph::OnSave()
{
	SpawnMissingNodesForParallel();
	UpdateAsset();
}

void UBehaviorTreeGraph::UpdatePinConnectionTypes()
{
	for (int32 Index = 0; Index < Nodes.Num(); ++Index)
	{
		UEdGraphNode* Node = Nodes[Index];
		check(Node);

		const bool bIsCompositeNode = Node->IsA(UBehaviorTreeGraphNode_Composite::StaticClass());

		for (int32 iPin = 0; iPin < Node->Pins.Num(); iPin++)
		{
			FName& PinCategory = Node->Pins[iPin]->PinType.PinCategory;
			if (PinCategory == TEXT("Transition"))
			{
				PinCategory = bIsCompositeNode ? 
					UBehaviorTreeEditorTypes::PinCategory_MultipleNodes :
					UBehaviorTreeEditorTypes::PinCategory_SingleComposite;
			}
		}
	}
}

void UBehaviorTreeGraph::ReplaceNodeConnections(UEdGraphNode* OldNode, UEdGraphNode* NewNode)
{
	for (int32 Index = 0; Index < Nodes.Num(); ++Index)
	{
		UEdGraphNode* Node = Nodes[Index];
		check(Node);
		for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); PinIndex++)
		{
			UEdGraphPin* Pin = Node->Pins[PinIndex];
			for (int32 LinkedIndex = 0; LinkedIndex < Pin->LinkedTo.Num(); LinkedIndex++)
			{
				UEdGraphPin* LinkedPin = Pin->LinkedTo[LinkedIndex];
				const UEdGraphNode* LinkedNode = LinkedPin ? LinkedPin->GetOwningNode() : NULL;

				if (LinkedNode == OldNode)
				{
					check(OldNode);
					check(LinkedPin);

					const int32 LinkedPinIndex = OldNode->Pins.IndexOfByKey(LinkedPin);
					Pin->LinkedTo[LinkedIndex] = NewNode->Pins[LinkedPinIndex];
					LinkedPin->LinkedTo.Remove(Pin);
				}
			}
		}
	}
}

void UBehaviorTreeGraph::UpdateVersion_RunBehaviorInSeparateGraph()
{
	for (int32 Index = 0; Index < Nodes.Num(); ++Index)
	{
		UBehaviorTreeGraphNode* Node = Cast<UBehaviorTreeGraphNode>(Nodes[Index]);
		if (Node)
		{
			// UBTTask_RunBehavior is now handled by dedicated graph node
			if (Node->NodeInstance && Node->NodeInstance->IsA(UBTTask_RunBehavior::StaticClass()))
			{
				UBehaviorTreeGraphNode* NewNode = Cast<UBehaviorTreeGraphNode>(StaticDuplicateObject(Node, this, NAME_None, RF_AllFlags, UBehaviorTreeGraphNode_SubtreeTask::StaticClass()));
				check(NewNode);

				ReplaceNodeConnections(Node, NewNode);
				Nodes[Index] = NewNode;

				Node->Rename(nullptr, GetTransientPackage());
				Node->SetFlags(RF_Transient);
				Node->MarkAsGarbage();
			}
		}
	}
}

void UBehaviorTreeGraph::RemoveUnknownSubNodes()
{
	for (int32 Index = 0; Index < Nodes.Num(); ++Index)
	{
		UBehaviorTreeGraphNode* Node = Cast<UBehaviorTreeGraphNode>(Nodes[Index]);
		if (Node)
		{
			for (int32 SubIdx = Node->SubNodes.Num() - 1; SubIdx >= 0; SubIdx--)
			{
				const bool bIsDecorator = Node->Decorators.Contains(Node->SubNodes[SubIdx]);
				const bool bIsService = Node->Services.Contains(Node->SubNodes[SubIdx]);

				if (!bIsDecorator && !bIsService)
				{
					Node->SubNodes.RemoveAt(SubIdx);
				}
			}
		}
	}
}

void UBehaviorTreeGraph::UpdateBrokenComposites()
{
	for (int32 Index = 0; Index < Nodes.Num(); ++Index)
	{
		UBehaviorTreeGraphNode_CompositeDecorator* Node = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(Nodes[Index]);
		if (Node)
		{
			Node->UpdateBrokenInstances();
		}
	}
}

namespace BTGraphHelpers
{
	struct FIntIntPair
	{
		int32 FirstIdx;
		int32 LastIdx;
	};

	void CollectDecorators(UBehaviorTree* BTAsset,
		UBehaviorTreeGraphNode* GraphNode, TArray<UBTDecorator*>& DecoratorInstances, TArray<FBTDecoratorLogic>& DecoratorOperations,
		bool bInitializeNodes, UBTCompositeNode* RootNode, uint16* ExecutionIndex, uint8 TreeDepth, int32 ChildIdx)
	{
		TMap<UBehaviorTreeGraphNode_CompositeDecorator*, FIntIntPair> CompositeMap;
		int32 NumNodes = 0;

		for (int32 i = 0; i < GraphNode->Decorators.Num(); i++)
		{
			UBehaviorTreeGraphNode* MyNode = GraphNode->Decorators[i];
			if (MyNode == NULL || MyNode->bInjectedNode)
			{
				continue;
			}

			UBehaviorTreeGraphNode_Decorator* MyDecoratorNode = Cast<UBehaviorTreeGraphNode_Decorator>(MyNode);
			UBehaviorTreeGraphNode_CompositeDecorator* MyCompositeNode = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(MyNode);

			if (MyDecoratorNode)
			{
				MyDecoratorNode->CollectDecoratorData(DecoratorInstances, DecoratorOperations);
				NumNodes++;
			}
			else if (MyCompositeNode)
			{
				MyCompositeNode->SetDecoratorData(RootNode, IntCastChecked<uint8>(ChildIdx));

				FIntIntPair RangeData;
				RangeData.FirstIdx = DecoratorInstances.Num();

				MyCompositeNode->CollectDecoratorData(DecoratorInstances, DecoratorOperations);
				NumNodes++;

				RangeData.LastIdx = DecoratorInstances.Num() - 1;
				CompositeMap.Add(MyCompositeNode, RangeData);
			}
		}

		for (int32 i = 0; i < DecoratorInstances.Num(); i++)
		{
			if (DecoratorInstances[i] && BTAsset && Cast<UBehaviorTree>(DecoratorInstances[i]->GetOuter()) == NULL)
			{
				DecoratorInstances[i]->Rename(NULL, BTAsset);
			}

			DecoratorInstances[i]->InitializeNode(RootNode, *ExecutionIndex, 0, TreeDepth);
			if (bInitializeNodes)
			{
				DecoratorInstances[i]->InitializeParentLink(IntCastChecked<uint8>(ChildIdx));
				*ExecutionIndex += 1;

				// make sure that flow abort mode matches - skip for root level nodes
				DecoratorInstances[i]->UpdateFlowAbortMode();
			}
		}

		if (bInitializeNodes)
		{
			// initialize composite decorators
			for (TMap<UBehaviorTreeGraphNode_CompositeDecorator*, FIntIntPair>::TIterator It(CompositeMap); It; ++It)
			{
				UBehaviorTreeGraphNode_CompositeDecorator* Node = It.Key();
				const FIntIntPair& PairInfo = It.Value();

				if (DecoratorInstances.IsValidIndex(PairInfo.FirstIdx) &&
					DecoratorInstances.IsValidIndex(PairInfo.LastIdx))
				{
					Node->FirstExecutionIndex = DecoratorInstances[PairInfo.FirstIdx]->GetExecutionIndex();
					Node->LastExecutionIndex = DecoratorInstances[PairInfo.LastIdx]->GetExecutionIndex();
				}
			}
		}

		// setup logic operations only when composite decorator is linked
		if (CompositeMap.Num())
		{
			if (NumNodes > 1)
			{
				FBTDecoratorLogic LogicOp(EBTDecoratorLogic::And, IntCastChecked<uint16>(NumNodes));
				DecoratorOperations.Insert(LogicOp, 0);
			}
		}
		else
		{
			DecoratorOperations.Reset();
		}
	}

	void InitializeInjectedNodes(UBehaviorTreeGraphNode* GraphNode, UBTCompositeNode* RootNode, uint16 ExecutionIndex, uint8 TreeDepth, int32 ChildIdx)
	{
		TMap<UBehaviorTreeGraphNode_CompositeDecorator*, FIntIntPair> CompositeMap;
		TArray<UBTDecorator*> DecoratorInstances;
		TArray<FBTDecoratorLogic> DecoratorOperations;

		for (int32 i = 0; i < GraphNode->Decorators.Num(); i++)
		{
			UBehaviorTreeGraphNode* MyNode = GraphNode->Decorators[i];
			if (MyNode == NULL || !MyNode->bInjectedNode)
			{
				continue;
			}

			UBehaviorTreeGraphNode_Decorator* MyDecoratorNode = Cast<UBehaviorTreeGraphNode_Decorator>(MyNode);
			UBehaviorTreeGraphNode_CompositeDecorator* MyCompositeNode = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(MyNode);

			if (MyDecoratorNode)
			{
				MyDecoratorNode->CollectDecoratorData(DecoratorInstances, DecoratorOperations);
			}
			else if (MyCompositeNode)
			{
				MyCompositeNode->SetDecoratorData(RootNode, IntCastChecked<uint8>(ChildIdx));

				FIntIntPair RangeData;
				RangeData.FirstIdx = DecoratorInstances.Num();

				MyCompositeNode->CollectDecoratorData(DecoratorInstances, DecoratorOperations);

				RangeData.LastIdx = DecoratorInstances.Num() - 1;
				CompositeMap.Add(MyCompositeNode, RangeData);
			}
		}

		for (int32 i = 0; i < DecoratorInstances.Num(); i++)
		{
			DecoratorInstances[i]->InitializeNode(RootNode, ExecutionIndex, 0, TreeDepth);
			DecoratorInstances[i]->InitializeParentLink(IntCastChecked<uint8>(ChildIdx));
			ExecutionIndex++;
		}

		// initialize composite decorators
		for (TMap<UBehaviorTreeGraphNode_CompositeDecorator*, FIntIntPair>::TIterator It(CompositeMap); It; ++It)
		{
			UBehaviorTreeGraphNode_CompositeDecorator* Node = It.Key();
			const FIntIntPair& PairInfo = It.Value();

			if (DecoratorInstances.IsValidIndex(PairInfo.FirstIdx) &&
				DecoratorInstances.IsValidIndex(PairInfo.LastIdx))
			{
				Node->FirstExecutionIndex = DecoratorInstances[PairInfo.FirstIdx]->GetExecutionIndex();
				Node->LastExecutionIndex = DecoratorInstances[PairInfo.LastIdx]->GetExecutionIndex();
			}
		}
	}

	void VerifyDecorators(UBehaviorTreeGraphNode* GraphNode)
	{
		TArray<UBTDecorator*> DecoratorInstances;
		TArray<FBTDecoratorLogic> DecoratorOperations;

		for (int32 i = 0; i < GraphNode->Decorators.Num(); i++)
		{
			UBehaviorTreeGraphNode* MyNode = GraphNode->Decorators[i];
			if (MyNode == NULL)
			{
				continue;
			}

			DecoratorInstances.Reset();
			DecoratorOperations.Reset();

			UBehaviorTreeGraphNode_Decorator* MyDecoratorNode = Cast<UBehaviorTreeGraphNode_Decorator>(MyNode);
			UBehaviorTreeGraphNode_CompositeDecorator* MyCompositeNode = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(MyNode);

			if (MyDecoratorNode)
			{
				MyDecoratorNode->CollectDecoratorData(DecoratorInstances, DecoratorOperations);
			}
			else if (MyCompositeNode)
			{
				MyCompositeNode->CollectDecoratorData(DecoratorInstances, DecoratorOperations);
			}

			MyNode->bHasObserverError = false;
			for (int32 SubIdx = 0; SubIdx < DecoratorInstances.Num(); SubIdx++)
			{
				MyNode->bHasObserverError = MyNode->bHasObserverError || !DecoratorInstances[SubIdx]->IsFlowAbortModeValid();
			}
		}
	}

	void CreateChildren(UBehaviorTree* BTAsset, UBTCompositeNode* RootNode, const UBehaviorTreeGraphNode* RootEdNode, uint16* ExecutionIndex, uint8 TreeDepth)
	{
		if (RootEdNode == NULL)
		{
			return;
		}

		RootNode->Children.Reset();
		RootNode->Services.Reset();

		// collect services (composite node)
		if (RootEdNode->Services.Num())
		{
			for (int32 ServiceIdx = 0; ServiceIdx < RootEdNode->Services.Num(); ServiceIdx++)
			{
				UBTService* ServiceInstance = RootEdNode->Services[ServiceIdx] ? Cast<UBTService>(RootEdNode->Services[ServiceIdx]->NodeInstance) : NULL;
				if (ServiceInstance)
				{
					if (Cast<UBehaviorTree>(ServiceInstance->GetOuter()) == NULL)
					{
						ServiceInstance->Rename(NULL, BTAsset);
					}
					ServiceInstance->InitializeNode(RootNode, *ExecutionIndex, 0, TreeDepth);
					*ExecutionIndex += 1;

					RootNode->Services.Add(ServiceInstance);
				}
			}
		}

		// gather all nodes
		int32 ChildIdx = 0;
		for (int32 PinIdx = 0; PinIdx < RootEdNode->Pins.Num(); PinIdx++)
		{
			UEdGraphPin* Pin = RootEdNode->Pins[PinIdx];
			if (Pin->Direction != EGPD_Output)
			{
				continue;
			}

			// sort connections so that they're organized the same as user can see in the editor
			Pin->LinkedTo.Sort(FCompareNodeXLocation());

			for (int32 Index = 0; Index < Pin->LinkedTo.Num(); ++Index)
			{
				UBehaviorTreeGraphNode* GraphNode = Cast<UBehaviorTreeGraphNode>(Pin->LinkedTo[Index]->GetOwningNode());
				if (GraphNode == NULL)
				{
					continue;
				}

				UBTTaskNode* TaskInstance = Cast<UBTTaskNode>(GraphNode->NodeInstance);
				if (TaskInstance && Cast<UBehaviorTree>(TaskInstance->GetOuter()) == NULL)
				{
					TaskInstance->Rename(NULL, BTAsset);
				}

				UBTCompositeNode* CompositeInstance = Cast<UBTCompositeNode>(GraphNode->NodeInstance);
				if (CompositeInstance && Cast<UBehaviorTree>(CompositeInstance->GetOuter()) == NULL)
				{
					CompositeInstance->Rename(NULL, BTAsset);
				}

				if (TaskInstance == NULL && CompositeInstance == NULL)
				{
					continue;
				}

				// collect decorators
				TArray<UBTDecorator*> DecoratorInstances;
				TArray<FBTDecoratorLogic> DecoratorOperations;
				CollectDecorators(BTAsset, GraphNode, DecoratorInstances, DecoratorOperations, true, RootNode, ExecutionIndex, TreeDepth, ChildIdx);

				// store child data
				FBTCompositeChild& ChildInfo = RootNode->Children.AddDefaulted_GetRef();
				ChildInfo.ChildComposite = CompositeInstance;
				ChildInfo.ChildTask = TaskInstance;
				ChildInfo.Decorators = DecoratorInstances;
				ChildInfo.DecoratorOps = DecoratorOperations;

				UBTNode* ChildNode = CompositeInstance ? (UBTNode*)CompositeInstance : (UBTNode*)TaskInstance;
				if (ChildNode && Cast<UBehaviorTree>(ChildNode->GetOuter()) == NULL)
				{
					ChildNode->Rename(NULL, BTAsset);
				}

				InitializeInjectedNodes(GraphNode, RootNode, *ExecutionIndex, TreeDepth, ChildIdx);
				
				// special case: subtrees
				UBTTask_RunBehavior* SubtreeTask = Cast<UBTTask_RunBehavior>(TaskInstance);
				if (SubtreeTask)
				{
					*ExecutionIndex += IntCastChecked<uint16>(SubtreeTask->GetInjectedNodesCount());
				}

				// collect services (task node)
				if (TaskInstance)
				{
					TaskInstance->Services.Reset();

					for (int32 ServiceIdx = 0; ServiceIdx < GraphNode->Services.Num(); ServiceIdx++)
					{
						UBTService* ServiceInstance = GraphNode->Services[ServiceIdx] ? Cast<UBTService>(GraphNode->Services[ServiceIdx]->NodeInstance) : NULL;
						if (ServiceInstance)
						{
							if (Cast<UBehaviorTree>(ServiceInstance->GetOuter()) == NULL)
							{
								ServiceInstance->Rename(NULL, BTAsset);
							}

							ServiceInstance->InitializeNode(RootNode, *ExecutionIndex, 0, TreeDepth);
							ServiceInstance->InitializeParentLink(IntCastChecked<uint8>(ChildIdx));
							*ExecutionIndex += 1;

							TaskInstance->Services.Add(ServiceInstance);
						}
					}
				}

				// assign execution index to child node
				ChildNode->InitializeNode(RootNode, *ExecutionIndex, 0, TreeDepth);
				*ExecutionIndex += 1;
				ChildIdx++;

				VerifyDecorators(GraphNode);

				if (CompositeInstance)
				{
					CreateChildren(BTAsset, CompositeInstance, GraphNode, ExecutionIndex, TreeDepth + 1);

					CompositeInstance->InitializeComposite((*ExecutionIndex) - 1);
				}
			}
		}
	}

	void ClearRootLevelFlags(UBehaviorTreeGraph* Graph)
	{
		for (int32 Index = 0; Index < Graph->Nodes.Num(); Index++)
		{
			UBehaviorTreeGraphNode* BTNode = Cast<UBehaviorTreeGraphNode>(Graph->Nodes[Index]);
			if (BTNode)
			{
				BTNode->bRootLevel = false;

				for (int32 SubIndex = 0; SubIndex < BTNode->Decorators.Num(); SubIndex++)
				{
					UBehaviorTreeGraphNode* SubNode = BTNode->Decorators[SubIndex];
					if (SubNode)
					{
						SubNode->bRootLevel = false;
					}
				}
			}
		}
	}

	void RebuildExecutionOrder(UBehaviorTreeGraphNode* RootEdNode, UBTCompositeNode* RootNode, uint16* ExecutionIndex, uint8 TreeDepth)
	{
		if (RootEdNode == NULL)
		{
			return;
		}

		// collect services: composite
		if (RootEdNode->Services.Num())
		{
			for (int32 i = 0; i < RootEdNode->Services.Num(); i++)
			{
				UBTService* ServiceInstance = RootEdNode->Services[i] ? Cast<UBTService>(RootEdNode->Services[i]->NodeInstance) : NULL;
				if (ServiceInstance)
				{
					ServiceInstance->InitializeNode(RootNode, *ExecutionIndex, 0, TreeDepth);
					*ExecutionIndex += 1;
				}
			}
		}

		// gather all nodes
		int32 ChildIdx = 0;
		for (int32 PinIdx = 0; PinIdx < RootEdNode->Pins.Num(); PinIdx++)
		{
			UEdGraphPin* Pin = RootEdNode->Pins[PinIdx];
			if (Pin->Direction != EGPD_Output)
			{
				continue;
			}

			// sort connections so that they're organized the same as user can see in the editor
			TArray<UEdGraphPin*> SortedPins = Pin->LinkedTo;
			SortedPins.Sort(FCompareNodeXLocation());

			for (int32 Index = 0; Index < SortedPins.Num(); ++Index)
			{
				UBehaviorTreeGraphNode* GraphNode = Cast<UBehaviorTreeGraphNode>(SortedPins[Index]->GetOwningNode());
				if (GraphNode == NULL)
				{
					continue;
				}

				UBTTaskNode* TaskInstance = Cast<UBTTaskNode>(GraphNode->NodeInstance);
				UBTCompositeNode* CompositeInstance = Cast<UBTCompositeNode>(GraphNode->NodeInstance);
				UBTNode* ChildNode = CompositeInstance ? (UBTNode*)CompositeInstance : (UBTNode*)TaskInstance;
				if (ChildNode == NULL)
				{
					continue;
				}

				// collect decorators
				TArray<UBTDecorator*> DecoratorInstances;
				TArray<FBTDecoratorLogic> DecoratorOperations;
				CollectDecorators(NULL, GraphNode, DecoratorInstances, DecoratorOperations, true, RootNode, ExecutionIndex, TreeDepth, ChildIdx);


				InitializeInjectedNodes(GraphNode, RootNode, *ExecutionIndex, TreeDepth, ChildIdx);

				// special case: subtrees
				UBTTask_RunBehavior* SubtreeTask = Cast<UBTTask_RunBehavior>(TaskInstance);
				if (SubtreeTask)
				{
					*ExecutionIndex += IntCastChecked<uint16>(SubtreeTask->GetInjectedNodesCount());
				}

				// collect services: task
				if (TaskInstance && GraphNode->Services.Num())
				{
					for (int32 ServiceIdx = 0; ServiceIdx < GraphNode->Services.Num(); ServiceIdx++)
					{
						UBTService* ServiceInstance = GraphNode->Services[ServiceIdx] ? Cast<UBTService>(GraphNode->Services[ServiceIdx]->NodeInstance) : NULL;
						if (ServiceInstance)
						{
							ServiceInstance->InitializeNode(RootNode, *ExecutionIndex, 0, TreeDepth);
							*ExecutionIndex += 1;
						}
					}
				}

				ChildNode->InitializeNode(RootNode, *ExecutionIndex, 0, TreeDepth);
				*ExecutionIndex += 1;
				ChildIdx++;

				if (CompositeInstance)
				{
					RebuildExecutionOrder(GraphNode, CompositeInstance, ExecutionIndex, TreeDepth + 1);
					CompositeInstance->InitializeComposite((*ExecutionIndex) - 1);
				}
			}
		}
	}

	UEdGraphPin* FindGraphNodePin(UEdGraphNode* Node, EEdGraphPinDirection Dir)
	{
		UEdGraphPin* Pin = nullptr;
		for (int32 Idx = 0; Idx < Node->Pins.Num(); Idx++)
		{
			if (Node->Pins[Idx]->Direction == Dir)
			{
				Pin = Node->Pins[Idx];
				break;
			}
		}

		return Pin;
	}

	void SpawnMissingDecoratorNodes(UBehaviorTreeGraphNode* GraphNode, const TArray<UBTDecorator*>& Decorators, const TArray<FBTDecoratorLogic>& DecoratorOps, UBehaviorTreeGraph* Graph)
	{
		if (DecoratorOps.Num() == 0)
		{
			for (int32 SubIdx = 0; SubIdx < Decorators.Num(); SubIdx++)
			{
				UBehaviorTreeGraphNode* DecoratorNode = NewObject<UBehaviorTreeGraphNode_Decorator>(Graph);
				GraphNode->AddSubNode(DecoratorNode, Graph);
				DecoratorNode->NodeInstance = Decorators[SubIdx];
			}
		}
		else
		{
			int32 Idx = 0;
			while (Idx < DecoratorOps.Num())
			{
				if (DecoratorOps[Idx].Operation == EBTDecoratorLogic::Test)
				{
					UBehaviorTreeGraphNode* DecoratorNode = NewObject<UBehaviorTreeGraphNode_Decorator>(Graph);
					GraphNode->AddSubNode(DecoratorNode, Graph);
					DecoratorNode->NodeInstance = Decorators[DecoratorOps[Idx].Number];

					Idx++;
				}
				else
				{
					UBehaviorTreeGraphNode_CompositeDecorator* CompositeNode = NewObject<UBehaviorTreeGraphNode_CompositeDecorator>(Graph);
					GraphNode->AddSubNode(CompositeNode, Graph);
					
					const int32 NextIdx = CompositeNode->SpawnMissingNodes(Decorators, DecoratorOps, Idx);
					CompositeNode->BuildDescription();

					Idx = NextIdx;
				}
			}
		}
	}

	UBehaviorTreeGraphNode* SpawnMissingGraphNodesWorker(UBTNode* Node, UBehaviorTreeGraphNode* ParentGraphNode, int32 ChildIdx, int32 ParentDecoratorCount, UBehaviorTreeGraph* Graph)
	{
		if (Node == nullptr)
		{
			return nullptr;
		}

		UBehaviorTreeGraphNode* GraphNode = nullptr;

		UBTCompositeNode* CompositeNode = Cast<UBTCompositeNode>(Node);
		if (Node->IsA(UBTComposite_SimpleParallel::StaticClass()))
		{
			FGraphNodeCreator<UBehaviorTreeGraphNode_SimpleParallel> NodeBuilder(*Graph);
			GraphNode = NodeBuilder.CreateNode();
			NodeBuilder.Finalize();
		}
		else if (CompositeNode)
		{
			FGraphNodeCreator<UBehaviorTreeGraphNode_Composite> NodeBuilder(*Graph);
			GraphNode = NodeBuilder.CreateNode();
			NodeBuilder.Finalize();
		}
		else if (Node->IsA(UBTTask_RunBehavior::StaticClass()))
		{
			FGraphNodeCreator<UBehaviorTreeGraphNode_SubtreeTask> NodeBuilder(*Graph);
			GraphNode = NodeBuilder.CreateNode();
			NodeBuilder.Finalize();
		}
		else
		{
			FGraphNodeCreator<UBehaviorTreeGraphNode_Task> NodeBuilder(*Graph);
			GraphNode = NodeBuilder.CreateNode();
			NodeBuilder.Finalize();
		}

		if (GraphNode)
		{
			const int32 ParentSubNodes = ParentGraphNode->Services.Num() + ParentGraphNode->Decorators.Num();
			GraphNode->NodePosX = ParentGraphNode->NodePosX + ChildIdx * 400;
			GraphNode->NodePosY = ParentGraphNode->NodePosY + (ParentDecoratorCount + ParentSubNodes + 1) * 75;
			GraphNode->NodeInstance = Node;
		}

		if (CompositeNode)
		{
			for (int32 SubIdx = 0; SubIdx < CompositeNode->Services.Num(); SubIdx++)
			{
				UBehaviorTreeGraphNode* ServiceNode = NewObject<UBehaviorTreeGraphNode_Service>(Graph);
				ServiceNode->NodeInstance = CompositeNode->Services[SubIdx];
				GraphNode->AddSubNode(ServiceNode, Graph);
			}

			UEdGraphPin* OutputPin = FindGraphNodePin(GraphNode, EGPD_Output);

			for (int32 Idx = 0; Idx < CompositeNode->Children.Num(); Idx++)
			{
				UBTNode* ChildNode = CompositeNode->GetChildNode(Idx);
				UBehaviorTreeGraphNode* ChildGraphNode = SpawnMissingGraphNodesWorker(ChildNode, GraphNode,
					Idx, ParentDecoratorCount + CompositeNode->Children[Idx].Decorators.Num(), Graph);

				SpawnMissingDecoratorNodes(ChildGraphNode,
					CompositeNode->Children[Idx].Decorators,
					CompositeNode->Children[Idx].DecoratorOps,
					Graph);

				UEdGraphPin* ChildInputPin = FindGraphNodePin(ChildGraphNode, EGPD_Input);

				OutputPin->MakeLinkTo(ChildInputPin);
			}
		}

		return GraphNode;
	}

	UBehaviorTreeGraphNode* SpawnMissingGraphNodes(UBehaviorTree* Asset, UBehaviorTreeGraphNode* ParentGraphNode, UBehaviorTreeGraph* Graph)
	{
		if (ParentGraphNode == nullptr || Asset == nullptr)
		{
			return nullptr;
		}

		UBehaviorTreeGraphNode* GraphNode = SpawnMissingGraphNodesWorker(Asset->RootNode, ParentGraphNode, 0, Asset->RootDecorators.Num(), Graph);
		SpawnMissingDecoratorNodes(GraphNode, Asset->RootDecorators, Asset->RootDecoratorOps, Graph);

		return GraphNode;
	}

} // namespace BTGraphHelpers

void UBehaviorTreeGraph::CreateBTFromGraph(UBehaviorTreeGraphNode* RootEdNode)
{
	UBehaviorTree* BTAsset = Cast<UBehaviorTree>(GetOuter());

	//discard old tree
	BTAsset->RootNode = nullptr;
	BTAsset->RootDecorators.Empty();
	BTAsset->RootDecoratorOps.Empty();

	if (RootEdNode != nullptr)
	{
		// let's create new tree from graph
		uint16 ExecutionIndex = 0;

		BTAsset->RootNode = Cast<UBTCompositeNode>(RootEdNode->NodeInstance);
		if (BTAsset->RootNode)
		{
			BTAsset->RootNode->InitializeNode(/*ParentNode*/nullptr, ExecutionIndex, /*MemoryOffset*/0 , /*TreeDepth*/0);
			ExecutionIndex++;
		}

		// collect root level decorators
		uint16 DummyExecutionIndex = MAX_uint16;
		BTGraphHelpers::CollectDecorators(BTAsset
			, RootEdNode
			, MutableView(BTAsset->RootDecorators)
			, BTAsset->RootDecoratorOps
			, /*bInitializeNodes*/false
			, /*RootNode*/nullptr
			, &DummyExecutionIndex
			, /*TreeDepth*/0
			, /*ChildIdx*/0);

		// connect tree nodes
		BTGraphHelpers::CreateChildren(BTAsset, BTAsset->RootNode, RootEdNode, &ExecutionIndex, /*TreeDepth*/1);

		// mark root level nodes
		BTGraphHelpers::ClearRootLevelFlags(this);

		RootEdNode->bRootLevel = true;
		for (int32 Index = 0; Index < RootEdNode->Decorators.Num(); Index++)
		{
			UBehaviorTreeGraphNode* Node = RootEdNode->Decorators[Index];
			if (Node)
			{
				Node->bRootLevel = true;
			}
		}

		if (BTAsset->RootNode)
		{
			BTAsset->RootNode->InitializeComposite(ExecutionIndex - 1);
		}
	}

	// Now remove any orphaned nodes left behind after regeneration
	RemoveOrphanedNodes();
}

void UBehaviorTreeGraph::CollectAllNodeInstances(TSet<UObject*>& NodeInstances)
{
	Super::CollectAllNodeInstances(NodeInstances);

	for (UEdGraphNode* EdGraphNode : Nodes)
	{
		UBehaviorTreeGraphNode* BTGraphNode = Cast<UBehaviorTreeGraphNode>(EdGraphNode);
		if (BTGraphNode != nullptr)
		{
			for (UBehaviorTreeGraphNode* BTDecoratorGraphNode : BTGraphNode->Decorators)
			{
				UEdGraph* EdSubGraph = BTDecoratorGraphNode->GetBoundGraph();
				if (EdSubGraph != nullptr)
				{
					for (UEdGraphNode* SubGraphNode : EdSubGraph->Nodes)
					{
						const UBehaviorTreeDecoratorGraphNode_Decorator* DecoratorNode = Cast<const UBehaviorTreeDecoratorGraphNode_Decorator>(SubGraphNode);
						if (DecoratorNode && DecoratorNode->NodeInstance)
						{
							NodeInstances.Add(DecoratorNode->NodeInstance);
						}
					}
				}
			}
		}
	}
}

void UBehaviorTreeGraph::SpawnMissingNodes()
{
	UBehaviorTree* BTAsset = Cast<UBehaviorTree>(GetOuter());
	if (BTAsset)
	{
		UBehaviorTreeGraphNode* RootNode = nullptr;
		for (int32 Idx = 0; Idx < Nodes.Num(); Idx++)
		{
			RootNode = Cast<UBehaviorTreeGraphNode_Root>(Nodes[Idx]);
			if (RootNode)
			{
				break;
			}
		}

		UBehaviorTreeGraphNode* SpawnedRootNode = BTGraphHelpers::SpawnMissingGraphNodes(BTAsset, RootNode, this);
		if (RootNode && SpawnedRootNode)
		{
			UEdGraphPin* RootOutPin = FindGraphNodePin(RootNode, EGPD_Output);
			UEdGraphPin* SpawnedInPin = FindGraphNodePin(SpawnedRootNode, EGPD_Input);

			RootOutPin->MakeLinkTo(SpawnedInPin);
		}
	}
}

void UBehaviorTreeGraph::SpawnMissingNodesForParallel()
{
	UBehaviorTree* BTAsset = Cast<UBehaviorTree>(GetOuter());
	if (BTAsset)
	{
		TArray<UBehaviorTreeGraphNode_SimpleParallel*> FixNodes;
		for (int32 Idx = 0; Idx < Nodes.Num(); Idx++)
		{
			UBehaviorTreeGraphNode_SimpleParallel* ParallelNode = Cast<UBehaviorTreeGraphNode_SimpleParallel>(Nodes[Idx]);
			if (ParallelNode)
			{
				UEdGraphPin* BackgroundPin = ParallelNode->GetOutputPin(1);
				if (BackgroundPin && BackgroundPin->LinkedTo.Num() == 0)
				{
					FixNodes.Add(ParallelNode);
				}
			}
		}

		for (int32 Idx = 0; Idx < FixNodes.Num(); Idx++)
		{
			UBehaviorTreeGraphNode_SimpleParallel* ParallelNode = FixNodes[Idx];
			UBTComposite_SimpleParallel* ParallelInstance = Cast<UBTComposite_SimpleParallel>(ParallelNode->NodeInstance);
			if (ParallelInstance)
			{
				int32 XOffset = 200;

				UEdGraphPin* MainTaskPin = ParallelNode->GetOutputPin(0);
				if (MainTaskPin && MainTaskPin->LinkedTo.Num())
				{
					UBehaviorTreeGraphNode* MainTaskNode = Cast<UBehaviorTreeGraphNode>(MainTaskPin->LinkedTo[0]->GetOwningNode());
					if (MainTaskNode)
					{
						const int32 Width = MainTaskNode->DEPRECATED_NodeWidget.IsValid() ? static_cast<int32>(MainTaskNode->DEPRECATED_NodeWidget.Pin()->GetDesiredSize().X) : 200;
						XOffset = MainTaskNode->NodePosX - ParallelNode->NodePosX + Width + 20;
					}
				}

				FGraphNodeCreator<UBehaviorTreeGraphNode_Task> NodeBuilder(*this);
				UBehaviorTreeGraphNode_Task* WaitTaskNode = NodeBuilder.CreateNode();
				WaitTaskNode->ClassData = FGraphNodeClassData(UBTTask_Wait::StaticClass(), "");
				NodeBuilder.Finalize();

				const int32 ParentHeight = ParallelNode->DEPRECATED_NodeWidget.IsValid() ? static_cast<int32>(ParallelNode->DEPRECATED_NodeWidget.Pin()->GetDesiredSize().Y) : 200;
				WaitTaskNode->NodePosX = ParallelNode->NodePosX + XOffset;
				WaitTaskNode->NodePosY = ParallelNode->NodePosY + ParentHeight + 20;

				UBTTask_Wait* WaitTaskInstance = Cast<UBTTask_Wait>(WaitTaskNode->NodeInstance);
				if (WaitTaskInstance)
				{
					WaitTaskInstance->WaitTime = (ParallelInstance->FinishMode == EBTParallelMode::WaitForBackground) ? 0.5f : 60.0f;
				}

				UEdGraphPin* BackgroundPin = ParallelNode->GetOutputPin(1);
				UEdGraphPin* InputPin = WaitTaskNode->GetInputPin();
				BackgroundPin->MakeLinkTo(InputPin);
			}
		}
	}
}

void UBehaviorTreeGraph::UpdateAbortHighlight(struct FAbortDrawHelper& Mode0, struct FAbortDrawHelper& Mode1)
{
	for (int32 Index = 0; Index < Nodes.Num(); ++Index)
	{
		UBehaviorTreeGraphNode* Node = Cast<UBehaviorTreeGraphNode>(Nodes[Index]);
		UBTNode* NodeInstance = Node ? Cast<UBTNode>(Node->NodeInstance) : NULL;
		if (NodeInstance)
		{
			const uint16 ExecIndex = NodeInstance->GetExecutionIndex();

			Node->bHighlightInAbortRange0 = (ExecIndex != MAX_uint16) && (ExecIndex >= Mode0.AbortStart) && (ExecIndex <= Mode0.AbortEnd);
			Node->bHighlightInAbortRange1 = (ExecIndex != MAX_uint16) && (ExecIndex >= Mode1.AbortStart) && (ExecIndex <= Mode1.AbortEnd);
			Node->bHighlightInSearchRange0 = (ExecIndex != MAX_uint16) && (ExecIndex >= Mode0.SearchStart) && (ExecIndex <= Mode0.SearchEnd);
			Node->bHighlightInSearchRange1 = (ExecIndex != MAX_uint16) && (ExecIndex >= Mode1.SearchStart) && (ExecIndex <= Mode1.SearchEnd);
			Node->bHighlightInSearchTree = false;
		}
	}
}

bool UBehaviorTreeGraph::UpdateInjectedNodes()
{
	bool bHasUpdated = false;
	for (int32 Index = 0; Index < Nodes.Num(); ++Index)
	{
		UBehaviorTreeGraphNode_SubtreeTask* Node = Cast<UBehaviorTreeGraphNode_SubtreeTask>(Nodes[Index]);
		if (Node)
		{
			const bool bUpdatedSubTree = Node->UpdateInjectedNodes();
			bHasUpdated = bHasUpdated || bUpdatedSubTree;
		}
	}

	return bHasUpdated;
}

UEdGraphNode* UBehaviorTreeGraph::FindInjectedNode(int32 Index)
{
	for (int32 NodeIdx = 0; NodeIdx < Nodes.Num(); NodeIdx++)
	{
		UBehaviorTreeGraphNode* MyNode = Cast<UBehaviorTreeGraphNode>(Nodes[NodeIdx]);
		if (MyNode && MyNode->bRootLevel)
		{
			return MyNode->Decorators.IsValidIndex(Index) ? MyNode->Decorators[Index] : NULL;
		}
	}

	return NULL;
}

void UBehaviorTreeGraph::RebuildExecutionOrder()
{
	// initial cleanup & root node search
	UBehaviorTreeGraphNode_Root* RootNode = NULL;
	for (int32 Index = 0; Index < Nodes.Num(); ++Index)
	{
		UBehaviorTreeGraphNode* Node = Cast<UBehaviorTreeGraphNode>(Nodes[Index]);
		if (Node == nullptr)
		{
			continue;;
		}

		// prepare node instance
		UBTNode* NodeInstance = Cast<UBTNode>(Node->NodeInstance);
		if (NodeInstance)
		{
			// mark all nodes as disconnected first, path from root will replace it with valid values later
			NodeInstance->InitializeNode(NULL, MAX_uint16, 0, 0);
		}

		// cache root
		if (RootNode == NULL)
		{
			RootNode = Cast<UBehaviorTreeGraphNode_Root>(Nodes[Index]);
		}

		UBehaviorTreeGraphNode_CompositeDecorator* CompositeDecorator = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(Nodes[Index]);
		if (CompositeDecorator)
		{
			CompositeDecorator->ResetExecutionRange();
		}
	}

	if (RootNode && RootNode->Pins.Num() > 0 && RootNode->Pins[0]->LinkedTo.Num() > 0)
	{
		UBehaviorTreeGraphNode* Node = Cast<UBehaviorTreeGraphNode>(RootNode->Pins[0]->LinkedTo[0]->GetOwningNode());
		if (Node)
		{
			UBTCompositeNode* BTNode = Cast<UBTCompositeNode>(Node->NodeInstance);
			if (BTNode)
			{
				uint16 ExecutionIndex = 0;
				uint8 TreeDepth = 0;

				BTNode->InitializeNode(NULL, ExecutionIndex, 0, TreeDepth);
				ExecutionIndex++;

				BTGraphHelpers::RebuildExecutionOrder(Node, BTNode, &ExecutionIndex, TreeDepth);
			}
		}
	}
}

void UBehaviorTreeGraph::RebuildChildOrder(UEdGraphNode* ParentNode)
{
	bool bUpdateExecutionOrder = false;
	if (ParentNode)
	{
		for (int32 PinIdx = 0; PinIdx < ParentNode->Pins.Num(); PinIdx++)
		{
			UEdGraphPin* Pin = ParentNode->Pins[PinIdx];
			if (Pin->Direction == EGPD_Output)
			{
				TArray<UEdGraphPin*> PrevOrder(Pin->LinkedTo);
				Pin->LinkedTo.Sort(FCompareNodeXLocation());

				bUpdateExecutionOrder = bUpdateExecutionOrder || (PrevOrder != Pin->LinkedTo);
			}
		}
	}

	if (bUpdateExecutionOrder)
	{
		UpdateAsset(KeepRebuildCounter);
		Modify();
	}
}

#if WITH_EDITOR
void UBehaviorTreeGraph::PostEditUndo()
{
	Super::PostEditUndo();

	// make sure that all execution indices are up to date
	UpdateAsset(KeepRebuildCounter);
	Modify();
}
#endif // WITH_EDITOR

namespace BTAutoArrangeHelpers
{
	struct FNodeBoundsInfo
	{
		FDeprecateSlateVector2D SubGraphBBox;
		TArray<FNodeBoundsInfo> Children;
	};

	void AutoArrangeNodes(UBehaviorTreeGraphNode* ParentNode, FNodeBoundsInfo& BBoxTree, float PosX, float PosY)
	{
		int32 BBoxIndex = 0;

		UEdGraphPin* Pin = BTGraphHelpers::FindGraphNodePin(ParentNode, EGPD_Output);
		if (Pin)
		{
			SGraphNode::FNodeSet NodeFilter;
			TArray<UEdGraphPin*> TempLinkedTo = Pin->LinkedTo;
			for (int32 Idx = 0; Idx < TempLinkedTo.Num(); Idx++)
			{
				UBehaviorTreeGraphNode* GraphNode = Cast<UBehaviorTreeGraphNode>(TempLinkedTo[Idx]->GetOwningNode());
				if (GraphNode && BBoxTree.Children.Num() > 0)
				{
					AutoArrangeNodes(GraphNode, BBoxTree.Children[BBoxIndex], PosX, PosY + GraphNode->DEPRECATED_NodeWidget.Pin()->GetDesiredSize().Y * 2.5f);
					GraphNode->DEPRECATED_NodeWidget.Pin()->MoveTo(FDeprecateSlateVector2D(BBoxTree.Children[BBoxIndex].SubGraphBBox.X / 2.f - GraphNode->DEPRECATED_NodeWidget.Pin()->GetDesiredSize().X / 2.f + PosX, PosY), NodeFilter);
					PosX += BBoxTree.Children[BBoxIndex].SubGraphBBox.X + 20.f;
					BBoxIndex++;
				}
			}
		}
	}

	void GetNodeSizeInfo(UBehaviorTreeGraphNode* ParentNode, FNodeBoundsInfo& BBoxTree)
	{
		BBoxTree.SubGraphBBox = ParentNode->DEPRECATED_NodeWidget.Pin()->GetDesiredSize();
		float LevelWidth = 0;
		float LevelHeight = 0;

		UEdGraphPin* Pin = BTGraphHelpers::FindGraphNodePin(ParentNode, EGPD_Output);
		if (Pin)
		{
			Pin->LinkedTo.Sort(FCompareNodeXLocation());
			for (int32 Idx = 0; Idx < Pin->LinkedTo.Num(); Idx++)
			{
				UBehaviorTreeGraphNode* GraphNode = Cast<UBehaviorTreeGraphNode>(Pin->LinkedTo[Idx]->GetOwningNode());
				if (GraphNode)
				{
					const int32 ChildIdx = BBoxTree.Children.Add(FNodeBoundsInfo());
					FNodeBoundsInfo& ChildBounds = BBoxTree.Children[ChildIdx];

					GetNodeSizeInfo(GraphNode, ChildBounds);

					LevelWidth += ChildBounds.SubGraphBBox.X + 20.f;
					if (ChildBounds.SubGraphBBox.Y > LevelHeight)
					{
						LevelHeight = ChildBounds.SubGraphBBox.Y;
					}
				}
			}

			if (LevelWidth > BBoxTree.SubGraphBBox.X)
			{
				BBoxTree.SubGraphBBox.X = LevelWidth;
			}

			BBoxTree.SubGraphBBox.Y += LevelHeight;
		}
	}
}

void UBehaviorTreeGraph::AutoArrange()
{
	UBehaviorTreeGraphNode* RootNode = nullptr;
	for (int32 Idx = 0; Idx < Nodes.Num(); Idx++)
	{
		RootNode = Cast<UBehaviorTreeGraphNode_Root>(Nodes[Idx]);
		if (RootNode)
		{
			break;
		}
	}

	if (!RootNode)
	{
		return;
	}

	BTAutoArrangeHelpers::FNodeBoundsInfo BBoxTree;
	BTAutoArrangeHelpers::GetNodeSizeInfo(RootNode, BBoxTree);
	BTAutoArrangeHelpers::AutoArrangeNodes(RootNode, BBoxTree, 0, RootNode->DEPRECATED_NodeWidget.Pin()->GetDesiredSize().Y * 2.5f);

	RootNode->NodePosX = static_cast<int32>(BBoxTree.SubGraphBBox.X / 2.f - RootNode->DEPRECATED_NodeWidget.Pin()->GetDesiredSize().X / 2.f);
	RootNode->NodePosY = 0;

	RootNode->DEPRECATED_NodeWidget.Pin()->GetOwnerPanel()->ZoomToFit(/*bOnlySelection=*/ false);
}

void UBehaviorTreeGraph::OnSubNodeDropped()
{
	Super::OnSubNodeDropped();

	FAbortDrawHelper EmptyMode;
	UpdateAsset(UBehaviorTreeGraph::ClearDebuggerFlags);
	UpdateAbortHighlight(EmptyMode, EmptyMode);
}

void UBehaviorTreeGraph::UpdateVersion()
{
	if (!bIsUsingModCounter)
	{
		bIsUsingModCounter = true;
		GraphVersion = BTGraphVersion::Initial;
	}

	if (GraphVersion == BTGraphVersion::Latest)
	{
		return;
	}

	// convert to nested nodes
	if (GraphVersion < BTGraphVersion::UnifiedSubNodes)
	{
		UpdateVersion_UnifiedSubNodes();
	}

	if (GraphVersion < BTGraphVersion::InnerGraphWhitespace)
	{
		UpdateVersion_InnerGraphWhitespace();
	}

	if (GraphVersion < BTGraphVersion::RunBehaviorInSeparateGraph)
	{
		UpdateVersion_RunBehaviorInSeparateGraph();
	}

	GraphVersion = BTGraphVersion::Latest;
	Modify();
}

void UBehaviorTreeGraph::MarkVersion()
{
	GraphVersion = BTGraphVersion::Latest;
	bIsUsingModCounter = true;
}

void UBehaviorTreeGraph::UpdateVersion_UnifiedSubNodes()
{
	for (int32 NodeIdx = 0; NodeIdx < Nodes.Num(); NodeIdx++)
	{
		UBehaviorTreeGraphNode* MyNode = Cast<UBehaviorTreeGraphNode>(Nodes[NodeIdx]);
		if (MyNode == nullptr)
		{
			continue;
		}

		MyNode->SubNodes.Reset(MyNode->Decorators.Num() + MyNode->Services.Num());

		for (int32 SubIdx = 0; SubIdx < MyNode->Decorators.Num(); SubIdx++)
		{
			MyNode->SubNodes.Add(MyNode->Decorators[SubIdx]);
		}

		for (int32 SubIdx = 0; SubIdx < MyNode->Services.Num(); SubIdx++)
		{
			MyNode->SubNodes.Add(MyNode->Services[SubIdx]);
		}
	}
}

void UBehaviorTreeGraph::UpdateVersion_InnerGraphWhitespace()
{
	for (int32 NodeIdx = 0; NodeIdx < Nodes.Num(); NodeIdx++)
	{
		UBehaviorTreeGraphNode* MyNode = Cast<UBehaviorTreeGraphNode>(Nodes[NodeIdx]);
		if (MyNode)
		{
			for (int32 SubIdx = 0; SubIdx < MyNode->SubNodes.Num(); SubIdx++)
			{
				UBehaviorTreeGraphNode_CompositeDecorator* InnerGraphNode = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(MyNode->SubNodes[SubIdx]);
				int32 DummyIdx = INDEX_NONE;
				
				if (InnerGraphNode && InnerGraphNode->BoundGraph && InnerGraphNode->BoundGraph->GetName().FindChar(TEXT(' '), DummyIdx))
				{
					// don't use white space in name here, it prevents links from being copied correctly
					InnerGraphNode->BoundGraph->Rename(TEXT("CompositeDecorator"), InnerGraphNode);
				}
			}
		}
	}
}

