// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeGraphNode.h"
#include "BehaviorTreeColors.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "ToolMenus.h"
#include "BehaviorTree/BTNode.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/BTService.h"
#include "BehaviorTreeEditorTypes.h"
#include "BehaviorTreeGraph.h"
#include "BehaviorTreeGraphNode_CompositeDecorator.h"
#include "BehaviorTreeGraphNode_Decorator.h"
#include "EdGraphSchema_BehaviorTree.h"
#include "SGraphEditorActionMenuAI.h"
#include "BehaviorTree/BehaviorTree.h"
#include "GraphDiffControl.h"
#include "Editor.h"
#include "SourceCodeNavigation.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeGraphNode)

#define LOCTEXT_NAMESPACE "BehaviorTreeEditor"

UBehaviorTreeGraphNode::UBehaviorTreeGraphNode(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	bHighlightInAbortRange0 = false;
	bHighlightInAbortRange1 = false;
	bHighlightInSearchRange0 = false;
	bHighlightInSearchRange1 = false;
	bHighlightInSearchTree = false;
	bHighlightChildNodeIndices = false;
	bRootLevel = false;
	bInjectedNode = false;
	bHasObserverError = false;
	bHasBreakpoint = false;
	bIsBreakpointEnabled = false;
	bDebuggerMarkCurrentlyActive = false;
	bDebuggerMarkPreviouslyActive = false;
	bDebuggerMarkFlashActive = false;
	bDebuggerMarkSearchSucceeded = false;
	bDebuggerMarkSearchFailed = false;
	bDebuggerMarkSearchTrigger = false;
	bDebuggerMarkSearchFailedTrigger = false;
	DebuggerSearchPathIndex = -1;
	DebuggerSearchPathSize = 0;
	DebuggerUpdateCounter = -1;
}

void UBehaviorTreeGraphNode::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UBehaviorTreeEditorTypes::PinCategory_MultipleNodes, TEXT("In"));
	CreatePin(EGPD_Output, UBehaviorTreeEditorTypes::PinCategory_MultipleNodes, TEXT("Out"));
}

void UBehaviorTreeGraphNode::InitializeInstance()
{
	UBTNode* BTNode = Cast<UBTNode>(NodeInstance);
	UBehaviorTree* BTAsset = BTNode ? Cast<UBehaviorTree>(BTNode->GetOuter()) : nullptr;
	if (BTNode && BTAsset)
	{
		BTNode->InitializeFromAsset(*BTAsset);
		BTNode->InitializeNode(NULL, MAX_uint16, 0, 0);
		BTNode->OnNodeCreated();
	}
}

FText UBehaviorTreeGraphNode::GetDescription() const
{
	const UBTNode* BTNode = Cast<const UBTNode>(NodeInstance);
	if (BTNode)
	{
		return FText::FromString(BTNode->GetStaticDescription());
	}

	return Super::GetDescription();
}

FText UBehaviorTreeGraphNode::GetTooltipText() const
{
	FText TooltipDesc;

	if (NodeInstance)
	{
		if (bHasObserverError)
		{
			TooltipDesc = LOCTEXT("ObserverError", "Observer has invalid abort setting!");
		}
	}

	if (TooltipDesc.IsEmpty())
	{
		TooltipDesc = Super::GetTooltipText();
	}

	if (bInjectedNode)
	{
		FText const InjectedDesc = !TooltipDesc.IsEmpty() ? TooltipDesc : GetDescription();
		// @TODO: FText::Format() is slow... consider caching this tooltip like 
		//        we do for a lot of the BP nodes now (unfamiliar with this 
		//        node's purpose, so hesitant to muck with this at this time).
		TooltipDesc = FText::Format(LOCTEXT("InjectedTooltip", "Injected: {0}"), InjectedDesc);
	}

	return TooltipDesc;
}

UBehaviorTreeGraph* UBehaviorTreeGraphNode::GetBehaviorTreeGraph()
{
	return GetOwnerBehaviorTreeGraph();
}

UBehaviorTreeGraph* UBehaviorTreeGraphNode::GetOwnerBehaviorTreeGraph() const
{
	return CastChecked<UBehaviorTreeGraph>(GetGraph());
}

bool UBehaviorTreeGraphNode::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const
{
	return DesiredSchema->GetClass()->IsChildOf(UEdGraphSchema_BehaviorTree::StaticClass());
}

void UBehaviorTreeGraphNode::FindDiffs(UEdGraphNode* OtherNode, FDiffResults& Results)
{
	Super::FindDiffs(OtherNode, Results);

	UBehaviorTreeGraphNode* OtherBTGraphNode = Cast<UBehaviorTreeGraphNode>(OtherNode);
	if (OtherBTGraphNode)
	{
		auto DiffSubNodes = [&Results](const FText& NodeTypeDisplayName, const TArray<UBehaviorTreeGraphNode*>& LhsSubNodes, const TArray<UBehaviorTreeGraphNode*>& RhsSubNodes)
		{
			TArray<FGraphDiffControl::FNodeMatch> NodeMatches;
			TSet<const UEdGraphNode*> MatchedRhsNodes;

			FGraphDiffControl::FNodeDiffContext AdditiveDiffContext;
			AdditiveDiffContext.NodeTypeDisplayName = NodeTypeDisplayName;
			AdditiveDiffContext.bIsRootNode = false;

			// march through the all the nodes in the rhs and look for matches 
			for (UEdGraphNode* RhsSubNode : RhsSubNodes)
			{
				FGraphDiffControl::FNodeMatch NodeMatch;
				NodeMatch.NewNode = RhsSubNode;

				// Do two passes, exact and soft
				for (UEdGraphNode* LhsSubNode : LhsSubNodes)
				{
					if (FGraphDiffControl::IsNodeMatch(LhsSubNode, RhsSubNode, true, &NodeMatches))
					{
						NodeMatch.OldNode = LhsSubNode;
						break;
					}
				}
				
				if (NodeMatch.NewNode == nullptr)
				{
					for (UEdGraphNode* LhsSubNode : LhsSubNodes)
					{
						if (FGraphDiffControl::IsNodeMatch(LhsSubNode, RhsSubNode, false, &NodeMatches))
						{
							NodeMatch.OldNode = LhsSubNode;
							break;
						}
					}
				}

				// if we found a corresponding node in the lhs graph, track it (so we can prevent future matches with the same nodes)
				if (NodeMatch.IsValid())
				{
					NodeMatches.Add(NodeMatch);
					MatchedRhsNodes.Add(NodeMatch.OldNode);
				}

				NodeMatch.Diff(AdditiveDiffContext, Results);
			}

			FGraphDiffControl::FNodeDiffContext SubtractiveDiffContext = AdditiveDiffContext;
			SubtractiveDiffContext.DiffMode = FGraphDiffControl::EDiffMode::Subtractive;
			SubtractiveDiffContext.DiffFlags = FGraphDiffControl::EDiffFlags::NodeExistance;

			// go through the lhs nodes to catch ones that may have been missing from the rhs graph
			for (UEdGraphNode* LhsSubNode : LhsSubNodes)
			{
				// if this node has already been matched, move on
				if (!LhsSubNode || MatchedRhsNodes.Find(LhsSubNode))
				{
					continue;
				}

				// There can't be a matching node in RhsGraph because it would have been found above
				FGraphDiffControl::FNodeMatch NodeMatch;
				NodeMatch.NewNode = LhsSubNode;

				NodeMatch.Diff(SubtractiveDiffContext, Results);
			}
		};

		DiffSubNodes(LOCTEXT("DecoratorDiffDisplayName", "Decorator"), Decorators, OtherBTGraphNode->Decorators);
		DiffSubNodes(LOCTEXT("ServiceDiffDisplayName", "Service"), Services, OtherBTGraphNode->Services);
	}
}

#if WITH_EDITOR
void UBehaviorTreeGraphNode::PostEditUndo()
{
	Super::PostEditUndo();

	UBehaviorTreeGraphNode* MyParentNode = Cast<UBehaviorTreeGraphNode>(ParentNode);
	if (MyParentNode)
	{
		const bool bIsDecorator = (Cast<UBTDecorator>(NodeInstance) != nullptr) || IsA(UBehaviorTreeGraphNode_CompositeDecorator::StaticClass());
		const bool bIsService = Cast<UBTService>(NodeInstance) != nullptr;

		if (bIsDecorator)
		{
			MyParentNode->Decorators.AddUnique(this);
		}
		else if (bIsService)
		{
			MyParentNode->Services.AddUnique(this);
		}
	}
}
#endif

void UBehaviorTreeGraphNode::OnSubNodeAdded(UAIGraphNode* NodeTemplate)
{
	UBehaviorTreeGraphNode* BTGraphNode = Cast<UBehaviorTreeGraphNode>(NodeTemplate);

	if (Cast<UBehaviorTreeGraphNode_CompositeDecorator>(NodeTemplate) || Cast<UBehaviorTreeGraphNode_Decorator>(NodeTemplate))
	{
		bool bAppend = true;
		for (int32 Idx = 0; Idx < Decorators.Num(); Idx++)
		{
			if (Decorators[Idx]->bInjectedNode)
			{
				Decorators.Insert(BTGraphNode, Idx);
				bAppend = false;
				break;
			}
		}

		if (bAppend)
		{
			Decorators.Add(BTGraphNode);
		}
	} 
	else
	{
		Services.Add(BTGraphNode);
	}
}

void UBehaviorTreeGraphNode::OnSubNodeRemoved(UAIGraphNode* SubNode)
{
	const int32 DecoratorIdx = Decorators.IndexOfByKey(SubNode);
	const int32 ServiceIdx = Services.IndexOfByKey(SubNode);

	if (DecoratorIdx >= 0)
	{
		Decorators.RemoveAt(DecoratorIdx);
	}

	if (ServiceIdx >= 0)
	{
		Services.RemoveAt(ServiceIdx);
	}
}

void UBehaviorTreeGraphNode::RemoveAllSubNodes()
{
	Super::RemoveAllSubNodes();

	Decorators.Reset();
	Services.Reset();
}

int32 UBehaviorTreeGraphNode::FindSubNodeDropIndex(UAIGraphNode* SubNode) const
{
	const int32 SubIdx = SubNodes.IndexOfByKey(SubNode) + 1;
	const int32 DecoratorIdx = Decorators.IndexOfByKey(SubNode) + 1;
	const int32 ServiceIdx = Services.IndexOfByKey(SubNode) + 1;

	const int32 CombinedIdx = (SubIdx & 0xff) | ((DecoratorIdx & 0xff) << 8) | ((ServiceIdx & 0xff) << 16);
	return CombinedIdx;
}

void UBehaviorTreeGraphNode::InsertSubNodeAt(UAIGraphNode* SubNode, int32 DropIndex)
{
	const int32 SubIdx = (DropIndex & 0xff) - 1;
	const int32 DecoratorIdx = ((DropIndex >> 8) & 0xff) - 1;
	const int32 ServiceIdx = ((DropIndex >> 16) & 0xff) - 1;

	if (SubIdx >= 0)
	{
		SubNodes.Insert(SubNode, SubIdx);
	}
	else
	{
		SubNodes.Add(SubNode);
	}

	UBehaviorTreeGraphNode* TypedNode = Cast<UBehaviorTreeGraphNode>(SubNode);
	const bool bIsDecorator = (Cast<UBTDecorator>(SubNode->NodeInstance) != nullptr) || (Cast<UBehaviorTreeGraphNode_CompositeDecorator>(SubNode) != nullptr);
	const bool bIsService = Cast<UBTService>(SubNode->NodeInstance) != nullptr;

	if (TypedNode)
	{
		if (bIsDecorator)
		{
			if (DecoratorIdx >= 0)
			{
				Decorators.Insert(TypedNode, DecoratorIdx);
			}
			else
			{
				Decorators.Add(TypedNode);
			}

		}

		if (bIsService)
		{
			if (ServiceIdx >= 0)
			{
				Services.Insert(TypedNode, ServiceIdx);
			}
			else
			{
				Services.Add(TypedNode);
			}
		}
	}
}

void UBehaviorTreeGraphNode::UpdateErrorMessage()
{
	Super::UpdateErrorMessage();

	if (ErrorMessage.IsEmpty())
	{
		if (const UBTNode* BTNodeInstance = Cast<UBTNode>(NodeInstance))
		{
			ErrorMessage = BTNodeInstance->GetErrorMessage();
		}
	}
}

FLinearColor UBehaviorTreeGraphNode::GetBackgroundColor(bool bIsActiveForDebugger) const
{
	return BehaviorTreeColors::NodeBody::Default;
}

void UBehaviorTreeGraphNode::CreateAddDecoratorSubMenu(UToolMenu* Menu, UEdGraph* Graph) const
{
	TSharedRef<SGraphEditorActionMenuAI> Widget =
		SNew(SGraphEditorActionMenuAI)
		.GraphObj( Graph )
		.GraphNode((UBehaviorTreeGraphNode*)this)
		.SubNodeFlags(ESubNode::Decorator)
		.AutoExpandActionMenu(true);

	FToolMenuSection& Section = Menu->FindOrAddSection("Section");
	Section.AddEntry(FToolMenuEntry::InitWidget("DecoratorWidget", Widget, FText(), true));
}

void UBehaviorTreeGraphNode::CreateAddServiceSubMenu(UToolMenu* Menu, UEdGraph* Graph) const
{
	TSharedRef<SGraphEditorActionMenuAI> Widget =
		SNew(SGraphEditorActionMenuAI)
		.GraphObj( Graph )
		.GraphNode((UBehaviorTreeGraphNode*)this)
		.SubNodeFlags(ESubNode::Service)
		.AutoExpandActionMenu(true);

	FToolMenuSection& Section = Menu->FindOrAddSection("Section");
	Section.AddEntry(FToolMenuEntry::InitWidget("ServiceWidget", Widget, FText(), true));
}

void UBehaviorTreeGraphNode::AddContextMenuActionsDecorators(UToolMenu* Menu, const FName SectionName, UGraphNodeContextMenuContext* Context) const
{
	FToolMenuSection& Section = Menu->FindOrAddSection(SectionName);
	Section.AddSubMenu(
		"AddDecorator",
		LOCTEXT("AddDecorator", "Add Decorator..." ),
		LOCTEXT("AddDecoratorTooltip", "Adds new decorator as a subnode" ),
		FNewToolMenuDelegate::CreateUObject(this, &UBehaviorTreeGraphNode::CreateAddDecoratorSubMenu, (UEdGraph*)Context->Graph));
}

void UBehaviorTreeGraphNode::AddContextMenuActionsServices(UToolMenu* Menu, const FName SectionName, UGraphNodeContextMenuContext* Context) const
{
	FToolMenuSection& Section = Menu->FindOrAddSection(SectionName);
	Section.AddSubMenu(
		"AddService",
		LOCTEXT("AddService", "Add Service..." ),
		LOCTEXT("AddServiceTooltip", "Adds new service as a subnode" ),
		FNewToolMenuDelegate::CreateUObject(this, &UBehaviorTreeGraphNode::CreateAddServiceSubMenu, (UEdGraph*)Context->Graph));
}

void UBehaviorTreeGraphNode::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	const UBTNode* BTNode = Cast<UBTNode>(NodeInstance);
	if (BTNode == nullptr)
	{
		return;
	}

	const UClass* NodeClass = BTNode->GetClass();
	if (NodeClass && !NodeClass->ClassGeneratedBy && FSourceCodeNavigation::CanNavigateToClass(NodeClass))
	{
		FToolMenuSection& Section = Menu->FindOrAddSection("BehaviorTreeGraphNode_SourceCode");
		Section.AddMenuEntry(
			"OpenSourceCode",
			FText::Format( LOCTEXT("OpenSourceCodeFor", "Open Source Code for {0}"), FText::FromString(NodeClass->GetAuthoredName())),
			LOCTEXT("OpenSourceCodeTooltip", "Open the C++ source file for this node"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.C++"),
			FUIAction(FExecuteAction::CreateLambda([NodeClass]()
			{
				FSourceCodeNavigation::NavigateToClass(NodeClass);
			}))
		);
	}
	else if (NodeClass && NodeClass->ClassGeneratedBy)
	{
		UObject* BlueprintAsset = NodeClass->ClassGeneratedBy;
		FToolMenuSection& Section = Menu->FindOrAddSection("BehaviorTreeGraphNode_Blueprint");
		Section.AddMenuEntry(
			"OpenBlueprint",
			FText::Format(LOCTEXT("OpenBlueprintFor", "Open Blueprint for {0}"), FText::FromString(NodeClass->GetAuthoredName())),
			LOCTEXT("OpenBlueprintTooltip", "Open the Blueprint asset for this node"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Blueprint"),
			FUIAction(FExecuteAction::CreateLambda([BlueprintAsset]()
			{
				if (BlueprintAsset)
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(BlueprintAsset);
				}
			}))
		);
	}
}

void UBehaviorTreeGraphNode::ClearDebuggerState()
{
	bHasBreakpoint = false;
	bIsBreakpointEnabled = false;
	bDebuggerMarkCurrentlyActive = false;
	bDebuggerMarkPreviouslyActive = false;
	bDebuggerMarkFlashActive = false;
	bDebuggerMarkSearchSucceeded = false;
	bDebuggerMarkSearchFailed = false;
	bDebuggerMarkSearchTrigger = false;
	bDebuggerMarkSearchFailedTrigger = false;
	DebuggerSearchPathIndex = -1;
	DebuggerSearchPathSize = 0;
	DebuggerUpdateCounter = -1;
	DebuggerRuntimeDescription.Empty();
}

const ISlateStyle& UBehaviorTreeGraphNode::GetNameIconStyleSet() const
{
	const UBTNode* BTNodeInstance = Cast<UBTNode>(NodeInstance);
	return BTNodeInstance != nullptr ? BTNodeInstance->GetNodeIconStyleSet() : FAppStyle::Get();
}

FName UBehaviorTreeGraphNode::GetNameIcon() const
{
	UBTNode* BTNodeInstance = Cast<UBTNode>(NodeInstance);
	return BTNodeInstance != nullptr ? BTNodeInstance->GetNodeIconName() : FName("BTEditor.Graph.BTNode.Icon");
}

bool UBehaviorTreeGraphNode::HasErrors() const
{
	return bHasObserverError || Super::HasErrors();
}

#undef LOCTEXT_NAMESPACE
