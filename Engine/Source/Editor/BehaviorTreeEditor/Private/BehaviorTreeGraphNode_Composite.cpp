// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeGraphNode_Composite.h"

#include "BehaviorTreeColors.h"
#include "BehaviorTreeGraph.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTree/BTNode.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeGraphNode_Composite)

class UToolMenu;

UBehaviorTreeGraphNode_Composite::UBehaviorTreeGraphNode_Composite(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UBehaviorTreeGraphNode_Composite::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	const UBTNode* MyNode = Cast<UBTNode>(NodeInstance);
	if (MyNode != NULL)
	{
		return FText::FromString(MyNode->GetNodeName());
	}
	return Super::GetNodeTitle(TitleType);
}

FText UBehaviorTreeGraphNode_Composite::GetDescription() const
{
	const UBTCompositeNode* CompositeNode = Cast<UBTCompositeNode>(NodeInstance);
	if (CompositeNode && CompositeNode->IsApplyingDecoratorScope())
	{
		return FText::Format(FText::FromString(TEXT("{0}\n{1}")),
			Super::GetDescription(),
			NSLOCTEXT("BehaviorTreeEditor", "CompositeNodeScopeDesc", "Local scope for observers"));
	}

	return Super::GetDescription();
}

FText UBehaviorTreeGraphNode_Composite::GetTooltipText() const
{
	const UBTCompositeNode* CompositeNode = Cast<UBTCompositeNode>(NodeInstance);
	if (CompositeNode && CompositeNode->IsApplyingDecoratorScope())
	{
		return FText::Format(FText::FromString(TEXT("{0}\n\n{1}")), 
			Super::GetDescription(), 
			NSLOCTEXT("BehaviorTreeEditor", "CompositeNodeScopeTooltip", "This node is a local scope for decorators.\nAll observing decorators (Lower Priority or Both) will be removed when execution flow leaves this branch."));
	}

	return Super::GetTooltipText();
}

FLinearColor UBehaviorTreeGraphNode_Composite::GetBackgroundColor(bool bIsActiveForDebugger) const
{
	UBTCompositeNode* CompositeNodeInstance = Cast<UBTCompositeNode>(NodeInstance);
	const bool bIsScoped = CompositeNodeInstance && CompositeNodeInstance->IsApplyingDecoratorScope();
	return bIsScoped ? BehaviorTreeColors::NodeBody::CompositeScoped : BehaviorTreeColors::NodeBody::Composite;
}

void UBehaviorTreeGraphNode_Composite::PostPasteNode()
{
	Super::PostPasteNode();
	
	// Clear our references to other nodes since this will be set when creating/updating the BT from the graph nodes
	UBTCompositeNode* CompositeNode = Cast<UBTCompositeNode>(NodeInstance);
	if (CompositeNode)
	{
		CompositeNode->Children.Reset();
		CompositeNode->Services.Reset();
	}
}

void UBehaviorTreeGraphNode_Composite::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	AddContextMenuActionsDecorators(Menu, "BehaviorTreeGraphNode", Context);

	if (GetOwnerBehaviorTreeGraph()->DoesSupportServices())
	{
		AddContextMenuActionsServices(Menu, "BehaviorTreeGraphNode", Context);
	}
}
