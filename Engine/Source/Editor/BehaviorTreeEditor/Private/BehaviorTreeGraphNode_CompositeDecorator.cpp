// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeGraphNode_CompositeDecorator.h"

#include "BehaviorTreeColors.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/BTNode.h"
#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTree/Decorators/BTDecorator_Blackboard.h"
#include "BehaviorTreeDecoratorGraph.h"
#include "BehaviorTreeDecoratorGraphNode_Decorator.h"
#include "Containers/EnumAsByte.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_BehaviorTreeDecorator.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeGraphNode_CompositeDecorator)

#define LOCTEXT_NAMESPACE "BehaviorTreeEditor"

UBehaviorTreeGraphNode_CompositeDecorator::UBehaviorTreeGraphNode_CompositeDecorator(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	bShowOperations = true;
	bCanAbortFlow = false;
	bHasBrokenInstances = false;

	FirstExecutionIndex = INDEX_NONE;
	LastExecutionIndex = INDEX_NONE;

	GraphClass = UBehaviorTreeDecoratorGraph::StaticClass();
}

void UBehaviorTreeGraphNode_CompositeDecorator::ResetExecutionRange()
{
	FirstExecutionIndex = INDEX_NONE;
	LastExecutionIndex = INDEX_NONE;
}

void UBehaviorTreeGraphNode_CompositeDecorator::AllocateDefaultPins()
{
	// No Pins for decorators
}

FString UBehaviorTreeGraphNode_CompositeDecorator::GetNodeTypeDescription() const
{
	return LOCTEXT("Composite","Composite").ToString();
}

FText UBehaviorTreeGraphNode_CompositeDecorator::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::FromString(CompositeName.Len() ? CompositeName : GetNodeTypeDescription());
}

FText UBehaviorTreeGraphNode_CompositeDecorator::GetDescription() const
{
	return FText::FromString(CachedDescription);
}

FText UBehaviorTreeGraphNode_CompositeDecorator::GetTooltipText() const
{
	if (ErrorMessage.IsEmpty() == false)
	{
		return FText::FromString(ErrorMessage);
	}

	return LOCTEXT("CompositeTooltip", "This node enables you to set up more advanced conditions using logic gates.");
}

void UBehaviorTreeGraphNode_CompositeDecorator::PostPlacedNewNode()
{
	if (BoundGraph == nullptr)
	{
		CreateBoundGraph();
	}

	Super::PostPlacedNewNode();
}

void UBehaviorTreeGraphNode_CompositeDecorator::PostLoad()
{
	Super::PostLoad();

	if (BoundGraph == nullptr)
	{
		CreateBoundGraph();
	}
}

void UBehaviorTreeGraphNode_CompositeDecorator::PrepareForCopying()
{
	Super::PrepareForCopying();
	
	if (BoundGraph)
	{
		for (int32 i = 0; i < BoundGraph->Nodes.Num(); i++)
		{
			UEdGraphNode* Node = BoundGraph->Nodes[i];
			Node->PrepareForCopying();
		}
	}
}

void UBehaviorTreeGraphNode_CompositeDecorator::PostCopyNode()
{
	Super::PostCopyNode();

	if (BoundGraph)
	{
		for (int32 i = 0; i < BoundGraph->Nodes.Num(); i++)
		{
			UBehaviorTreeDecoratorGraphNode_Decorator* Node = Cast<UBehaviorTreeDecoratorGraphNode_Decorator>(BoundGraph->Nodes[i]);
			if (Node)
			{
				Node->PostCopyNode();
			}
		}
	}
}

void UBehaviorTreeGraphNode_CompositeDecorator::PostPasteNode()
{
	Super::PostPasteNode();

	// Clear reference to the parent since it will be set when creating/updating the BT from the graph nodes
	ParentNodeInstance = nullptr;
}

void UBehaviorTreeGraphNode_CompositeDecorator::ResetNodeOwner()
{
	Super::ResetNodeOwner();

	if (BoundGraph)
	{
		for (int32 i = 0; i < BoundGraph->Nodes.Num(); i++)
		{
			UBehaviorTreeDecoratorGraphNode_Decorator* Node = Cast<UBehaviorTreeDecoratorGraphNode_Decorator>(BoundGraph->Nodes[i]);
			if (Node)
			{
				Node->ResetNodeOwner();
			}
		}
	}
}

bool UBehaviorTreeGraphNode_CompositeDecorator::RefreshNodeClass()
{
	bool bUpdated = false;

	if (BoundGraph)
	{
		for (int32 i = 0; i < BoundGraph->Nodes.Num(); i++)
		{
			UBehaviorTreeDecoratorGraphNode_Decorator* Node = Cast<UBehaviorTreeDecoratorGraphNode_Decorator>(BoundGraph->Nodes[i]);
			if (Node)
			{
				const bool bNodeUpdated = Node->RefreshNodeClass();
				bUpdated = bUpdated || bNodeUpdated;
			}
		}
	}
	
	return bUpdated;
}

void UBehaviorTreeGraphNode_CompositeDecorator::UpdateNodeClassData()
{
	if (BoundGraph)
	{
		for (int32 i = 0; i < BoundGraph->Nodes.Num(); i++)
		{
			UBehaviorTreeDecoratorGraphNode_Decorator* Node = Cast<UBehaviorTreeDecoratorGraphNode_Decorator>(BoundGraph->Nodes[i]);
			if (Node)
			{
				Node->UpdateNodeClassData();
			}
		}
	}
}

void UBehaviorTreeGraphNode_CompositeDecorator::UpdateBrokenInstances()
{
	bHasBrokenInstances = false;
	if (BoundGraph)
	{
		for (int32 i = 0; i < BoundGraph->Nodes.Num(); i++)
		{
			UBehaviorTreeDecoratorGraphNode_Decorator* Node = Cast<UBehaviorTreeDecoratorGraphNode_Decorator>(BoundGraph->Nodes[i]);
			if (Node && Node->NodeInstance == nullptr)
			{
				bHasBrokenInstances = true;
				break;
			}
		}
	}
}

bool UBehaviorTreeGraphNode_CompositeDecorator::HasErrors() const
{
	return bHasObserverError || bHasBrokenInstances;
}

void UBehaviorTreeGraphNode_CompositeDecorator::CreateBoundGraph()
{
	// Create a new animation graph
	check(BoundGraph == NULL);

	const TSubclassOf<UEdGraphSchema> SchemaClass = GetDefault<UBehaviorTreeDecoratorGraph>(GraphClass)->Schema;
	check(SchemaClass);

	// don't use white space in name here, it prevents links from being copied correctly
	BoundGraph = FBlueprintEditorUtils::CreateNewGraph(this, TEXT("CompositeDecorator"), GraphClass, SchemaClass);
	check(BoundGraph);

	// Initialize the anim graph
	const UEdGraphSchema* Schema = BoundGraph->GetSchema();
	Schema->CreateDefaultNodesForGraph(*BoundGraph);

	// Add the new graph as a child of our parent graph
	GetGraph()->SubGraphs.Add(BoundGraph);
}

bool UBehaviorTreeGraphNode_CompositeDecorator::IsSubNode() const
{
	return true;
}

void UBehaviorTreeGraphNode_CompositeDecorator::CollectDecoratorData(TArray<UBTDecorator*>& NodeInstances, TArray<FBTDecoratorLogic>& Operations) const
{
	const UBehaviorTreeDecoratorGraph* MyGraph = Cast<const UBehaviorTreeDecoratorGraph>(BoundGraph);
	if (MyGraph)
	{
		MyGraph->CollectDecoratorData(NodeInstances, Operations);
	}
}

void UBehaviorTreeGraphNode_CompositeDecorator::SetDecoratorData(class UBTCompositeNode* InParentNode, uint8 InChildIndex)
{
	ParentNodeInstance = InParentNode;
	ChildIndex = InChildIndex;
}

void UBehaviorTreeGraphNode_CompositeDecorator::InitializeDecorator(class UBTDecorator* InnerDecorator)
{
	InnerDecorator->InitializeNode(ParentNodeInstance, 0, 0, 0);
	InnerDecorator->InitializeParentLink(ChildIndex);
}

void UBehaviorTreeGraphNode_CompositeDecorator::OnBlackboardUpdate()
{
	UBehaviorTreeDecoratorGraph* MyGraph = Cast<UBehaviorTreeDecoratorGraph>(BoundGraph);
	UBehaviorTree* BTAsset = Cast<UBehaviorTree>(GetOuter()->GetOuter());
	if (MyGraph && BTAsset)
	{
		for (int32 i = 0; i < MyGraph->Nodes.Num(); i++)
		{
			UBehaviorTreeDecoratorGraphNode_Decorator* MyNode = Cast<UBehaviorTreeDecoratorGraphNode_Decorator>(MyGraph->Nodes[i]);
			UBTNode* MyNodeInstance = MyNode ? Cast<UBTNode>(MyNode->NodeInstance) : NULL;
			if (MyNodeInstance)
			{
				MyNodeInstance->InitializeFromAsset(*BTAsset);
			}
		}
	}
}

void UBehaviorTreeGraphNode_CompositeDecorator::OnInnerGraphChanged()
{
	BuildDescription();

	bCanAbortFlow = false;

	UBehaviorTreeDecoratorGraph* MyGraph = Cast<UBehaviorTreeDecoratorGraph>(BoundGraph);
	if (MyGraph)
	{
		for (int32 i = 0; i < MyGraph->Nodes.Num(); i++)
		{
			UBehaviorTreeDecoratorGraphNode_Decorator* MyNode = Cast<UBehaviorTreeDecoratorGraphNode_Decorator>(MyGraph->Nodes[i]);
			UBTDecorator* MyNodeInstance = MyNode ? Cast<UBTDecorator>(MyNode->NodeInstance) : NULL;
			if (MyNodeInstance && MyNodeInstance->GetFlowAbortMode() != EBTFlowAbortMode::None)
			{
				bCanAbortFlow = true;
				break;
			}
		}
	}
}

int32 UBehaviorTreeGraphNode_CompositeDecorator::SpawnMissingNodes(const TArray<class UBTDecorator*>& NodeInstances, const TArray<struct FBTDecoratorLogic>& Operations, int32 StartIndex)
{
	int32 FirstIdxOutside = StartIndex + 1;
	
	UBehaviorTreeDecoratorGraph* MyGraph = Cast<UBehaviorTreeDecoratorGraph>(BoundGraph);
	if (MyGraph)
	{
		FirstIdxOutside = MyGraph->SpawnMissingNodes(NodeInstances, Operations, StartIndex);
	}

	return FirstIdxOutside;
}

void UBehaviorTreeGraphNode_CompositeDecorator::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property &&
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UBehaviorTreeGraphNode_CompositeDecorator, bShowOperations))
	{
		BuildDescription();
	}
}

FLinearColor UBehaviorTreeGraphNode_CompositeDecorator::GetBackgroundColor(bool bIsActiveForDebugger) const
{
	return bIsActiveForDebugger
		? BehaviorTreeColors::Debugger::ActiveDecorator
		: (bInjectedNode || bRootLevel)
			? BehaviorTreeColors::NodeBody::InjectedSubNode
			: BehaviorTreeColors::NodeBody::Decorator;
}

struct FLogicDesc
{
	FString OperationDesc;
	int32 NumLeft;
};

void UpdateLogicOpStack(TArray<FLogicDesc>& OpStack, FString& Description, FString& Indent)
{
	if (OpStack.Num())
	{
		const int32 LastIdx = OpStack.Num() - 1;
		OpStack[LastIdx].NumLeft = OpStack[LastIdx].NumLeft - 1;

		if (OpStack[LastIdx].NumLeft <= 0)
		{
			OpStack.RemoveAt(LastIdx);
			Indent.LeftChopInline(2, EAllowShrinking::No);

			UpdateLogicOpStack(OpStack, Description, Indent);
		}
		else
		{
			Description += Indent;
			Description += OpStack[LastIdx].OperationDesc;
		}
	}
}

void UBehaviorTreeGraphNode_CompositeDecorator::BuildDescription()
{
	FString BaseDesc("Composite Decorator");
	if (!bShowOperations)
	{
		CachedDescription = BaseDesc;
		return;
	}

	TArray<UBTDecorator*> NodeInstances;
	TArray<FBTDecoratorLogic> Operations;
	CollectDecoratorData(NodeInstances, Operations);

	TArray<FLogicDesc> OpStack;
	FString Description = BaseDesc + TEXT(":");
	FString Indent("\n");
	bool bPendingNotOp = false;

	for (int32 i = 0; i < Operations.Num(); i++)
	{
		const FBTDecoratorLogic& TestOp = Operations[i];
		if (TestOp.Operation == EBTDecoratorLogic::And ||
			TestOp.Operation == EBTDecoratorLogic::Or)
		{
			Indent += TEXT("- ");

			FLogicDesc NewOpDesc;
			NewOpDesc.NumLeft = TestOp.Number;
			NewOpDesc.OperationDesc = (TestOp.Operation == EBTDecoratorLogic::And) ? TEXT("AND") : TEXT("OR");
			
			OpStack.Add(NewOpDesc);
		}
		else if (TestOp.Operation == EBTDecoratorLogic::Not)
		{
			// special case: NOT before TEST
			if (Operations.IsValidIndex(i + 1) && Operations[i + 1].Operation == EBTDecoratorLogic::Test)
			{
				bPendingNotOp = true;
			}
			else
			{
				Indent += TEXT("- ");
				Description += Indent;
				Description += TEXT("NOT:");

				FLogicDesc NewOpDesc;
				NewOpDesc.NumLeft = 0;

				OpStack.Add(NewOpDesc);
			}
		}
		else if (TestOp.Operation == EBTDecoratorLogic::Test)
		{
			Description += Indent;
			if (bPendingNotOp)
			{
				Description += TEXT("NOT: ");
				bPendingNotOp = false;
			}

			// Composite decorator based on blackboard might need to rebuild inner decorators description before aggregating it
			if (UBTDecorator_Blackboard* Decorator = Cast<UBTDecorator_Blackboard>(NodeInstances[TestOp.Number]))
			{
				Decorator->BuildDescription();
			}
			Description += NodeInstances[TestOp.Number]->GetStaticDescription();
			UpdateLogicOpStack(OpStack, Description, Indent);
		}
	}

	CachedDescription = Description;
}

#undef LOCTEXT_NAMESPACE
