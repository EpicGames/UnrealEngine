// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeGraphNode_Decorator.h"

#include "AIGraphTypes.h"
#include "BehaviorTreeColors.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTree/BTDecorator.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeGraphNode_Decorator)

UBehaviorTreeGraphNode_Decorator::UBehaviorTreeGraphNode_Decorator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsSubNode = true;
}

void UBehaviorTreeGraphNode_Decorator::AllocateDefaultPins()
{
	//No Pins for decorators
}

FLinearColor UBehaviorTreeGraphNode_Decorator::GetBackgroundColor(bool bIsActiveForDebugger) const
{
	return bIsActiveForDebugger
		? BehaviorTreeColors::Debugger::ActiveDecorator
		: (bInjectedNode || bRootLevel)
			? BehaviorTreeColors::NodeBody::InjectedSubNode
			: BehaviorTreeColors::NodeBody::Decorator;
}

FText UBehaviorTreeGraphNode_Decorator::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	const UBTDecorator* Decorator = Cast<UBTDecorator>(NodeInstance);
	if (Decorator != NULL)
	{
		return FText::FromString(Decorator->GetNodeName());
	}
	else if (!ClassData.GetClassName().IsEmpty())
	{
		FString StoredClassName = ClassData.GetClassName();
		StoredClassName.RemoveFromEnd(TEXT("_C"));

		return FText::Format(NSLOCTEXT("AIGraph", "NodeClassError", "Class {0} not found, make sure it's saved!"), FText::FromString(StoredClassName));
	}

	return Super::GetNodeTitle(TitleType);
}

void UBehaviorTreeGraphNode_Decorator::CollectDecoratorData(TArray<UBTDecorator*>& NodeInstances, TArray<FBTDecoratorLogic>& Operations) const
{
	if (NodeInstance)
	{
		UBTDecorator* DecoratorNode = (UBTDecorator*)NodeInstance;
		const int32 InstanceIdx = NodeInstances.Add(DecoratorNode);
		Operations.Add(FBTDecoratorLogic(EBTDecoratorLogic::Test, IntCastChecked<uint16>(InstanceIdx)));
	}
}
