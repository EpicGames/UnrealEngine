// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeGraphNode_Root.h"

#include "AISystem.h"
#include "BehaviorTreeColors.h"
#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTree/BlackboardData.h"
#include "BehaviorTreeEditorTypes.h"
#include "BehaviorTreeGraph.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeGraphNode_Root)

UBehaviorTreeGraphNode_Root::UBehaviorTreeGraphNode_Root(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	bIsReadOnly = true;
}

void UBehaviorTreeGraphNode_Root::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	const TSoftObjectPtr<UBlackboardData>& DefaultBlackboard = GET_AI_CONFIG_VAR(DefaultBlackboard);
	if (DefaultBlackboard.IsValid())
	{
		if (UBlackboardData* Blackboard = DefaultBlackboard.LoadSynchronous())
		{
			BlackboardAsset = Blackboard;
			UpdateBlackboard();
			return;
		}
	}

	// pick first available blackboard asset, hopefully something will be loaded...
	for (FThreadSafeObjectIterator It(UBlackboardData::StaticClass()); It; ++It)
	{
		UBlackboardData* TestOb = (UBlackboardData*)*It;
		if (!TestOb->HasAnyFlags(RF_ClassDefaultObject))
		{
			BlackboardAsset = TestOb;
			UpdateBlackboard();
			break;
		}
	}
}

void UBehaviorTreeGraphNode_Root::AllocateDefaultPins()
{
	CreatePin(EGPD_Output, UBehaviorTreeEditorTypes::PinCategory_SingleComposite, TEXT("In"));
}

FText UBehaviorTreeGraphNode_Root::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("BehaviorTreeEditor", "Root", "ROOT");
}

FName UBehaviorTreeGraphNode_Root::GetNameIcon() const
{
	return FName("BTEditor.Graph.BTNode.Root.Icon");
}

FText UBehaviorTreeGraphNode_Root::GetTooltipText() const
{
	return UEdGraphNode::GetTooltipText();
}

void UBehaviorTreeGraphNode_Root::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property &&
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UBehaviorTreeGraphNode_Root, BlackboardAsset))
	{
		UpdateBlackboard();
	}
}

void UBehaviorTreeGraphNode_Root::PostEditUndo()
{
	Super::PostEditUndo();
	UpdateBlackboard();
}

FText UBehaviorTreeGraphNode_Root::GetDescription() const
{
	return FText::FromString(GetNameSafe(BlackboardAsset));
}

FLinearColor UBehaviorTreeGraphNode_Root::GetBackgroundColor(bool bIsActiveForDebugger) const
{
	if (Pins.IsValidIndex(0) && Pins[0]->LinkedTo.Num() > 0)
	{
		return BehaviorTreeColors::NodeBody::Root;
	}

	return Super::GetBackgroundColor(bIsActiveForDebugger);
}

void UBehaviorTreeGraphNode_Root::UpdateBlackboard()
{
	UBehaviorTreeGraph* MyGraph = GetOwnerBehaviorTreeGraph();
	UBehaviorTree* BTAsset = Cast<UBehaviorTree>(MyGraph->GetOuter());
	if (BTAsset && BTAsset->BlackboardAsset != BlackboardAsset)
	{
		BTAsset->BlackboardAsset = BlackboardAsset;
		MyGraph->UpdateBlackboardChange();
	}
}
