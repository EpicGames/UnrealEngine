// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeGraphNode_Service.h"

#include "AIGraphTypes.h"
#include "BehaviorTreeColors.h"
#include "BehaviorTree/BTService.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeGraphNode_Service)

UBehaviorTreeGraphNode_Service::UBehaviorTreeGraphNode_Service(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsSubNode = true;
}

void UBehaviorTreeGraphNode_Service::AllocateDefaultPins()
{
	//No Pins for services
}

FLinearColor UBehaviorTreeGraphNode_Service::GetBackgroundColor(bool bIsActiveForDebugger) const
{
	return bIsActiveForDebugger ? BehaviorTreeColors::Debugger::ActiveService : BehaviorTreeColors::NodeBody::Service;
}


FText UBehaviorTreeGraphNode_Service::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	const UBTService* Service = Cast<UBTService>(NodeInstance);
	if (Service != NULL)
	{
		return FText::FromString(Service->GetNodeName());
	}
	else if (!ClassData.GetClassName().IsEmpty())
	{
		FString StoredClassName = ClassData.GetClassName();
		StoredClassName.RemoveFromEnd(TEXT("_C"));

		return FText::Format(NSLOCTEXT("AIGraph", "NodeClassError", "Class {0} not found, make sure it's saved!"), FText::FromString(StoredClassName));
	}

	return Super::GetNodeTitle(TitleType);
}
