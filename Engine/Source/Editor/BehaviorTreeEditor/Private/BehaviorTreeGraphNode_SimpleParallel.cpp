// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeGraphNode_SimpleParallel.h"

#include "BehaviorTreeEditorTypes.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeGraphNode_SimpleParallel)

#define LOCTEXT_NAMESPACE "BehaviorTreeEditor"

UBehaviorTreeGraphNode_SimpleParallel::UBehaviorTreeGraphNode_SimpleParallel(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

void UBehaviorTreeGraphNode_SimpleParallel::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UBehaviorTreeEditorTypes::PinCategory_MultipleNodes, TEXT("In"));
	
	CreatePin(EGPD_Output, UBehaviorTreeEditorTypes::PinCategory_SingleTask, TEXT("Task"));
	CreatePin(EGPD_Output, UBehaviorTreeEditorTypes::PinCategory_SingleNode, TEXT("Out"));
}

void UBehaviorTreeGraphNode_SimpleParallel::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	ensure(Pin.GetOwningNode() == this);
	if (Pin.Direction == EGPD_Output)
	{
		if (Pin.PinType.PinCategory == UBehaviorTreeEditorTypes::PinCategory_SingleTask)
		{
			HoverTextOut = LOCTEXT("PinHoverParallelMain","Main task of parrallel node").ToString();
		}
		else
		{
			HoverTextOut = LOCTEXT("PinHoverParallelBackground","Nodes running in the background, while main task is active").ToString();
		}
	}
}

#undef LOCTEXT_NAMESPACE
