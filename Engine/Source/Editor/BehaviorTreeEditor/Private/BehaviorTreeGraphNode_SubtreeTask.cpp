// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeGraphNode_SubtreeTask.h"

#include "BehaviorTreeColors.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTree/Tasks/BTTask_RunBehavior.h"
#include "BehaviorTreeDecoratorGraphNode_Decorator.h"
#include "BehaviorTreeGraph.h"
#include "BehaviorTreeGraphNode.h"
#include "BehaviorTreeGraphNode_CompositeDecorator.h"
#include "BehaviorTreeGraphNode_Root.h"
#include "Containers/Array.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/PlatformCrt.h"
#include "Templates/Casts.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeGraphNode_SubtreeTask)

UBehaviorTreeGraphNode_SubtreeTask::UBehaviorTreeGraphNode_SubtreeTask(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	SubtreeVersion = 0;
}

bool UBehaviorTreeGraphNode_SubtreeTask::UpdateInjectedNodes()
{
	bool bUpdated = false;

	// check if cached data needs to be updated
	UBTTask_RunBehavior* MyNode = Cast<UBTTask_RunBehavior>(NodeInstance);
	if (MyNode == NULL)
	{
		return bUpdated;
	}

	UBehaviorTreeGraph* MyGraph = MyNode->GetSubtreeAsset() ? Cast<UBehaviorTreeGraph>(MyNode->GetSubtreeAsset()->BTGraph) : NULL;
	int32 MyVersion = MyGraph ? MyGraph->ModCounter : 0;
	FString MyPath = MyNode->GetSubtreeAsset() ? MyNode->GetSubtreeAsset()->GetName() : FString();

	if (MyPath == SubtreePath && MyVersion == SubtreeVersion)
	{
		return bUpdated;
	}

	SubtreePath = MyPath;
	SubtreeVersion = MyVersion;
	bUpdated = true;

	// remove existing injected nodes
	for (int32 Index = Decorators.Num() - 1; Index >= 0; Index--)
	{
		if (Decorators[Index] && Decorators[Index]->bInjectedNode)
		{
			SubNodes.RemoveSingle(Decorators[Index]);
			Decorators.RemoveAt(Index, EAllowShrinking::No);
		}
	}

	// find root graph node of subtree
	UBehaviorTreeGraphNode* SubRoot = NULL;
	if (MyGraph && MyNode->GetSubtreeAsset()->RootDecorators.Num())
	{
		for (int32 Index = 0; Index < MyGraph->Nodes.Num(); Index++)
		{
			UBehaviorTreeGraphNode_Root* BTNode = Cast<UBehaviorTreeGraphNode_Root>(MyGraph->Nodes[Index]);
			if (BTNode && BTNode->Pins.IsValidIndex(0) && BTNode->Pins[0]->LinkedTo.IsValidIndex(0))
			{
				SubRoot = Cast<UBehaviorTreeGraphNode>(BTNode->Pins[0]->LinkedTo[0]->GetOwningNode());
				break;
			}
		}
	}

	// add root level subnodes as injected nodes
	if (SubRoot)
	{
		UBehaviorTree* BTAsset = Cast<UBehaviorTree>(GetOwnerBehaviorTreeGraph()->GetOuter());

		if(BTAsset)
		{
			for (int32 Index = 0; Index < SubRoot->Decorators.Num(); Index++)
			{
				UBehaviorTreeGraphNode* SubNode = SubRoot->Decorators[Index];
				if (SubNode)
				{
					SubNode->PrepareForCopying();

					UBehaviorTreeGraphNode* InjectedNode = Cast<UBehaviorTreeGraphNode>(StaticDuplicateObject(SubNode, GetOuter()));

					SubNode->PostCopyNode();
					InjectedNode->PostCopyNode();

					InjectedNode->ParentNode = this;
					InjectedNode->bInjectedNode = true;
					InjectedNode->bIsReadOnly = true;

					UBTDecorator* InjectedInstance = Cast<UBTDecorator>(InjectedNode->NodeInstance);
					if (InjectedInstance)
					{
						InjectedInstance->InitializeFromAsset(*BTAsset);
					}

					UBehaviorTreeGraphNode_CompositeDecorator* CompNode = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(InjectedNode);
					if (CompNode)
					{
						UEdGraph* SubGraph = CompNode->GetBoundGraph();
						if (SubGraph)
						{
							SubGraph->bEditable = false;

							for (int32 SubIndex = 0; SubIndex < SubGraph->Nodes.Num(); SubIndex++)
							{
								UBehaviorTreeDecoratorGraphNode_Decorator* InjectedDecorator = Cast<UBehaviorTreeDecoratorGraphNode_Decorator>(SubGraph->Nodes[SubIndex]);
								if (InjectedDecorator)
								{
									InjectedInstance = Cast<UBTDecorator>(InjectedDecorator->NodeInstance);
									if (InjectedInstance)
									{
										InjectedInstance->InitializeFromAsset(*BTAsset);
									}
								}
							}
						}
					}

					SubNodes.Add(InjectedNode);
					Decorators.Add(InjectedNode);
				}
			}
		}
	}

	UEdGraph* ParentGraph = GetGraph();
	if (ParentGraph && bUpdated)
	{
		ParentGraph->NotifyGraphChanged();
	}

	return bUpdated;
}

FLinearColor UBehaviorTreeGraphNode_SubtreeTask::GetBackgroundColor(bool bIsActiveForDebugger) const
{
	return BehaviorTreeColors::NodeBody::TaskSpecial;
}
