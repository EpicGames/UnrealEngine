// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeGraphNode_Task.h"

#include "AIGraphTypes.h"
#include "BehaviorTreeColors.h"
#include "BehaviorTree/BTNode.h"
#include "BehaviorTreeEditorTypes.h"
#include "BehaviorTreeGraph.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeGraphNode_Task)

class UToolMenu;

UBehaviorTreeGraphNode_Task::UBehaviorTreeGraphNode_Task(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

void UBehaviorTreeGraphNode_Task::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UBehaviorTreeEditorTypes::PinCategory_SingleComposite, TEXT("In"));
}

FText UBehaviorTreeGraphNode_Task::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	const UBTNode* MyNode = Cast<UBTNode>(NodeInstance);
	if (MyNode != NULL)
	{
		return FText::FromString(MyNode->GetNodeName());
	}
	else if (!ClassData.GetClassName().IsEmpty())
	{
		FString StoredClassName = ClassData.GetClassName();
		StoredClassName.RemoveFromEnd(TEXT("_C"));

		return FText::Format(NSLOCTEXT("AIGraph", "NodeClassError", "Class {0} not found, make sure it's saved!"), FText::FromString(StoredClassName));
	}

	return Super::GetNodeTitle(TitleType);
}

void UBehaviorTreeGraphNode_Task::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	AddContextMenuActionsDecorators(Menu, "BehaviorTreeGraphNode", Context);

	if (GetOwnerBehaviorTreeGraph()->DoesSupportServices())
	{
		AddContextMenuActionsServices(Menu, "BehaviorTreeGraphNode", Context);
	}
}

FLinearColor UBehaviorTreeGraphNode_Task::GetBackgroundColor(bool bIsActiveForDebugger) const
{
	return BehaviorTreeColors::NodeBody::Task;
}
