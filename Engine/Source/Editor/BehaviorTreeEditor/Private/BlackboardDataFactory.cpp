// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackboardDataFactory.h"

#include "BehaviorTree/BlackboardData.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardDataFactory)

class FFeedbackContext;
class UObject;

#define LOCTEXT_NAMESPACE "BlackboardDataFactory"

UBlackboardDataFactory::UBlackboardDataFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UBlackboardData::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

bool UBlackboardDataFactory::CanCreateNew() const
{
	return true;
}

UObject* UBlackboardDataFactory::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UBlackboardData::StaticClass()));
	return NewObject<UBlackboardData>(InParent, Class, Name, Flags);
}

#undef LOCTEXT_NAMESPACE
