// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailCustomizations/BehaviorDecoratorDetails.h"

#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTreeDebugger.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformCrt.h"
#include "IDetailPropertyRow.h"
#include "IPropertyUtilities.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "SlateOptMacros.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "BehaviorDecoratorDetails"

TSharedRef<IDetailCustomization> FBehaviorDecoratorDetails::MakeInstance()
{
	return MakeShareable( new FBehaviorDecoratorDetails );
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FBehaviorDecoratorDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	FText AbortModeDesc = LOCTEXT("ObserverTitle","Observer aborts");
	PropUtils = &(DetailLayout.GetPropertyUtilities().Get());

	TArray<TWeakObjectPtr<UObject> > EditedObjects;
	DetailLayout.GetObjectsBeingCustomized(EditedObjects);
	
	for (int32 i = 0; i < EditedObjects.Num(); i++)
	{
		UBTDecorator* MyDecorator = Cast<UBTDecorator>(EditedObjects[i].Get());
		if (MyDecorator)
		{
			MyNode = MyDecorator;
			break;
		}
	}
	
	UpdateAllowedAbortModes();
	ModeProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UBTDecorator, FlowAbortMode));

	// dynamic FlowAbortMode combo
	IDetailCategoryBuilder& FlowCategory = DetailLayout.EditCategory( "FlowControl" );
	IDetailPropertyRow& AbortModeRow = FlowCategory.AddProperty(ModeProperty);
	AbortModeRow.IsEnabled(TAttribute<bool>(this, &FBehaviorDecoratorDetails::GetAbortModeEnabled));
	AbortModeRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FBehaviorDecoratorDetails::GetModeVisibility)));
	AbortModeRow.CustomWidget()
		.NameContent()
		[
			ModeProperty->CreatePropertyNameWidget(AbortModeDesc)
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FBehaviorDecoratorDetails::OnGetAbortModeContent)
 			.ContentPadding(FMargin( 2.0f, 2.0f ))
			.ButtonContent()
			[
				SNew(STextBlock) 
				.Text(this, &FBehaviorDecoratorDetails::GetCurrentAbortModeDesc)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
	InitPropertyValues();

}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FBehaviorDecoratorDetails::UpdateAllowedAbortModes()
{
	ModeValues.Reset();

	UBTDecorator* MyDecorator = Cast<UBTDecorator>(MyNode);
	UBTCompositeNode* MyParentNode = MyDecorator ? MyDecorator->GetParentNode() : NULL;

	const bool bAllowAbortNone = MyDecorator == NULL || MyDecorator->bAllowAbortNone;
	const bool bAllowAbortSelf = (MyDecorator == NULL || MyDecorator->bAllowAbortChildNodes) && (MyParentNode == NULL || MyParentNode->CanAbortSelf());
	const bool bAllowAbortLowerPriority = (MyDecorator == NULL || MyDecorator->bAllowAbortLowerPri) && (MyParentNode == NULL || MyParentNode->CanAbortLowerPriority());

	const bool AbortCondition[] = { bAllowAbortNone, bAllowAbortSelf, bAllowAbortLowerPriority, bAllowAbortSelf && bAllowAbortLowerPriority };
	EBTFlowAbortMode::Type AbortValues[] = { EBTFlowAbortMode::None, EBTFlowAbortMode::Self, EBTFlowAbortMode::LowerPriority, EBTFlowAbortMode::Both };
	for (int32 i = 0; i < 4; i++)
	{
		if (AbortCondition[i])
		{
			FStringIntPair ModeDesc;
			ModeDesc.Int = AbortValues[i];
			ModeDesc.Str = UBehaviorTreeTypes::DescribeFlowAbortMode(AbortValues[i]);
			ModeValues.Add(ModeDesc);
		}
	}

	bIsModeEnabled = MyDecorator && ModeValues.Num();
	bShowMode = ModeValues.Num() > 0;
}

bool FBehaviorDecoratorDetails::IsEditingEnabled() const
{
	return FBehaviorTreeDebugger::IsPIENotSimulating() && PropUtils->IsPropertyEditingEnabled();
}

bool FBehaviorDecoratorDetails::GetAbortModeEnabled() const
{
	return bIsModeEnabled && IsEditingEnabled();
}

void FBehaviorDecoratorDetails::InitPropertyValues()
{
	uint8 ByteValue;
	ModeProperty->GetValue(ByteValue);
	OnAbortModeChange(ByteValue);
}

EVisibility FBehaviorDecoratorDetails::GetModeVisibility() const
{
	return bShowMode ? EVisibility::Visible : EVisibility::Collapsed;
}

void FBehaviorDecoratorDetails::OnAbortModeChange(int32 Index)
{
	uint8 ByteValue = (uint8)Index;
	ModeProperty->SetValue(ByteValue);
}

TSharedRef<SWidget> FBehaviorDecoratorDetails::OnGetAbortModeContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);

	for (int32 i = 0; i < ModeValues.Num(); i++)
	{
		FUIAction ItemAction( FExecuteAction::CreateSP( const_cast<FBehaviorDecoratorDetails*>(this), &FBehaviorDecoratorDetails::OnAbortModeChange, ModeValues[i].Int ) );
		MenuBuilder.AddMenuEntry(FText::FromString( ModeValues[i].Str ), TAttribute<FText>(), FSlateIcon(), ItemAction);
	}

	return MenuBuilder.MakeWidget();
}

FText FBehaviorDecoratorDetails::GetCurrentAbortModeDesc() const
{
	uint8 ByteValue;
	ModeProperty->GetValue(ByteValue);

	for (int32 i = 0; i < ModeValues.Num(); i++)
	{
		if (ModeValues[i].Int == ByteValue)
		{
			return FText::FromString(ModeValues[i].Str);
		}
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
