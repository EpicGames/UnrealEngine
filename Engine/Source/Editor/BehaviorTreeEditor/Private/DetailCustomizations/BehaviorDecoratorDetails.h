// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;

class FBehaviorDecoratorDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

	bool IsEditingEnabled() const;

private:

	void UpdateAllowedAbortModes();
	void InitPropertyValues();

	EVisibility GetModeVisibility() const;
	bool GetAbortModeEnabled() const;

	void OnAbortModeChange(int32 Index);
	TSharedRef<SWidget> OnGetAbortModeContent() const;
	FText GetCurrentAbortModeDesc() const;

	TSharedPtr<IPropertyHandle> ModeProperty;

	/** cached names of abort behaviors */
	struct FStringIntPair
	{
		FString Str;
		int32 Int;
	};

	TArray<FStringIntPair> ModeValues;

	uint32 bShowMode : 1;
	uint32 bIsModeEnabled : 1;

	class UObject* MyNode;
	/** property utils */
	class IPropertyUtilities* PropUtils;
};
