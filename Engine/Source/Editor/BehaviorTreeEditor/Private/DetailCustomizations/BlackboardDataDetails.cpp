// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailCustomizations/BlackboardDataDetails.h"

#include "BehaviorTree/BlackboardData.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "SBehaviorTreeBlackboardEditor.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "BlackboardDataDetails"

TSharedRef<IDetailCustomization> FBlackboardDataDetails::MakeInstance(FOnGetSelectedBlackboardItemIndex InOnGetSelectedBlackboardItemIndex, UBlackboardData* InBlackboardData)
{
	return MakeShareable(new FBlackboardDataDetails(InOnGetSelectedBlackboardItemIndex, InBlackboardData));
}

void FBlackboardDataDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	// First hide all keys
	DetailLayout.HideProperty(TEXT("Keys"));
	DetailLayout.HideProperty(TEXT("ParentKeys"));

	// Now show only the currently selected key
	bool bIsInherited = false;
	int32 CurrentSelection = INDEX_NONE;
	if (OnGetSelectedBlackboardItemIndex.IsBound())
	{
		CurrentSelection = OnGetSelectedBlackboardItemIndex.Execute(bIsInherited);
	}

	if (CurrentSelection >= 0)
	{
		TSharedPtr<IPropertyHandle> KeysHandle = bIsInherited ? DetailLayout.GetProperty(TEXT("ParentKeys")) : DetailLayout.GetProperty(TEXT("Keys"));
		check(KeysHandle.IsValid());
		uint32 NumChildKeys = 0;
		KeysHandle->GetNumChildren(NumChildKeys);
		if ((uint32)CurrentSelection < NumChildKeys)
		{
			KeyHandle = KeysHandle->GetChildHandle((uint32)CurrentSelection);

			IDetailCategoryBuilder& DetailCategoryBuilder = DetailLayout.EditCategory("Key");
			TSharedPtr<IPropertyHandle> EntryNameProperty = KeyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBlackboardEntry, EntryName));
			DetailCategoryBuilder.AddCustomRow(LOCTEXT("EntryNameLabel", "Entry Name"))
			.NameContent()
			[
				EntryNameProperty->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				.IsEnabled(true)
				+SHorizontalBox::Slot()
				[
					EntryNameProperty->CreatePropertyValueWidget()
				]
			];

			PopulateKeyCategories();

			const FText CategoryTooltip = LOCTEXT("BlackboardDataDetails_EditCategoryName_Tooltip", "The category of the variable; editing this will place the variable into another category or create a new one.");
			TSharedPtr<IPropertyHandle> EntryCategoryProperty = KeyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBlackboardEntry, EntryCategory));

			DetailCategoryBuilder.AddCustomRow(LOCTEXT("BlackboardDataDetails_EntryCategoryLabel", "Entry Category"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("BlackboardDataDetails_EntryCategoryLabel", "Entry Category"))
				.ToolTipText(CategoryTooltip)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SAssignNew(KeyCategoryComboButton, SComboButton)
				.IsEnabled(EntryCategoryProperty.IsValid() && !EntryCategoryProperty->IsEditConst())
				.ContentPadding(FMargin(0, 0, 5, 0))
				.ButtonContent()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					.Padding(FMargin(0, 0, 5, 0))
					[
						SNew(SEditableTextBox)
						.Text(this, &FBlackboardDataDetails::OnGetKeyCategoryText)
						.OnTextCommitted(this, &FBlackboardDataDetails::OnKeyCategoryTextCommitted)
						.ToolTipText(CategoryTooltip)
						.SelectAllTextWhenFocused(true)
						.RevertTextOnEscape(true)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
				.MenuContent()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.MaxHeight(400.0f)
					[
						SAssignNew(KeyCategoryListView, SListView<TSharedPtr<FText>>)
						.ListItemsSource(&KeyCategorySource)
						.OnGenerateRow(this, &FBlackboardDataDetails::MakeKeyCategoryViewWidget)
						.OnSelectionChanged(this, &FBlackboardDataDetails::OnKeyCategorySelectionChanged)
					]
				]
			];

			TSharedPtr<IPropertyHandle> EntryDescriptionHandle = KeyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBlackboardEntry, EntryDescription));
			DetailCategoryBuilder.AddProperty(EntryDescriptionHandle);

			TSharedPtr<IPropertyHandle> KeyTypeProperty = KeyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBlackboardEntry, KeyType));
			DetailCategoryBuilder.AddProperty(KeyTypeProperty)
			.EditCondition(!bIsInherited, nullptr); /** nullptr because the bool condition will never change so no need for a listener. */

			TSharedPtr<IPropertyHandle> bInstanceSyncedProperty = KeyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBlackboardEntry, bInstanceSynced));
			DetailCategoryBuilder.AddProperty(bInstanceSyncedProperty);
		}	
	}
}

FText FBlackboardDataDetails::OnGetKeyCategoryText() const
{
	FName PropertyCategoryText;
	
	check(KeyHandle.IsValid())
	TSharedPtr<IPropertyHandle> EntryCategoryProperty = KeyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBlackboardEntry, EntryCategory));	
	if (EntryCategoryProperty.IsValid())
	{
		EntryCategoryProperty->GetValue(PropertyCategoryText);
	}
	return FText::FromName(PropertyCategoryText);
}

void FBlackboardDataDetails::OnKeyCategoryTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	check(KeyHandle.IsValid())
	if (InTextCommit == ETextCommit::OnEnter || InTextCommit == ETextCommit::OnUserMovedFocus)
	{ 
		TSharedPtr<IPropertyHandle> EntryCategoryProperty = KeyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBlackboardEntry, EntryCategory));
		if (EntryCategoryProperty.IsValid())
		{
			EntryCategoryProperty->SetValue(FName(*InNewText.ToString()));
		}
		PopulateKeyCategories();
	}
}

TSharedRef<ITableRow> FBlackboardDataDetails::MakeKeyCategoryViewWidget(TSharedPtr<FText> Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		[
			SNew(STextBlock)
			.Text(*Item.Get())
		];
}

void FBlackboardDataDetails::OnKeyCategorySelectionChanged(TSharedPtr<FText> ProposedSelection, ESelectInfo::Type /*SelectInfo*/)
{
	check(KeyHandle.IsValid()); 
	if (ProposedSelection.IsValid())
	{
		FText NewCategory = *ProposedSelection.Get(); 
		TSharedPtr<IPropertyHandle> EntryCategoryProperty = KeyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBlackboardEntry, EntryCategory));
		if (EntryCategoryProperty.IsValid())
		{
			EntryCategoryProperty->SetValue(FName(*NewCategory.ToString()));
		}
		check(KeyCategoryListView.IsValid());
		check(KeyCategoryComboButton.IsValid()); 
		KeyCategoryListView->ClearSelection();
		KeyCategoryComboButton->SetIsOpen(false);
	}
}

void FBlackboardDataDetails::PopulateKeyCategories()
{ 
	KeyCategorySource.Reset();
	KeyCategorySource.Add(MakeShareable(new FText(LOCTEXT("None", "None"))));
	if (!BlackboardData.IsValid())
	{ 
		UE_LOGF(LogBlackboardEditor, Error, "Unable to populate variable categories without a valid blackboard asset.");
		return;
	}

	TArray<FBlackboardEntry> AllKeys;
	AllKeys.Append(BlackboardData->ParentKeys);
	AllKeys.Append(BlackboardData->Keys);
	
	TArray<FName> UniqueCategories;
	for (const FBlackboardEntry& Entry : AllKeys)
	{
		if (!Entry.EntryCategory.IsNone())
		{
			UniqueCategories.AddUnique(Entry.EntryCategory);
		}
	}

	for (const FName& Category : UniqueCategories)
	{
		KeyCategorySource.Add(MakeShareable(new FText(FText::FromName(Category))));
	}
}

#undef LOCTEXT_NAMESPACE
