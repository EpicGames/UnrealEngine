// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTree/BlackboardData.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Views/SListView.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class ITableRow;
class SComboButton;
class STableViewBase;

/** Delegate used to retrieve current blackboard selection */
DECLARE_DELEGATE_RetVal_OneParam(int32, FOnGetSelectedBlackboardItemIndex, bool& /* bIsInherited */);

class FBlackboardDataDetails : public IDetailCustomization
{
public:
	FBlackboardDataDetails(FOnGetSelectedBlackboardItemIndex InOnGetSelectedBlackboardItemIndex, UBlackboardData* InBlackboardData)
		: OnGetSelectedBlackboardItemIndex(InOnGetSelectedBlackboardItemIndex)
		, BlackboardData(InBlackboardData)
	{}

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance(FOnGetSelectedBlackboardItemIndex InOnGetSelectedBlackboardItemIndex, UBlackboardData* InBlackboardData);

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

private:
	FText OnGetKeyCategoryText() const;
	void OnKeyCategoryTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);
	void OnKeyCategorySelectionChanged(TSharedPtr<FText> ProposedSelection, ESelectInfo::Type /*SelectInfo*/);
	TSharedRef<ITableRow> MakeKeyCategoryViewWidget(TSharedPtr<FText> Item, const TSharedRef< STableViewBase >& OwnerTable);
	void PopulateKeyCategories();

private:
	/** Delegate used to retrieve current blackboard selection */
	FOnGetSelectedBlackboardItemIndex			OnGetSelectedBlackboardItemIndex;
	TSharedPtr<IPropertyHandle>					KeyHandle;
	TSharedPtr<SListView<TSharedPtr<FText>>>	KeyCategoryListView;
	TSharedPtr<SComboButton>					KeyCategoryComboButton;
	TArray<TSharedPtr<FText>>					KeyCategorySource;
	TWeakObjectPtr<UBlackboardData>				BlackboardData;
};
