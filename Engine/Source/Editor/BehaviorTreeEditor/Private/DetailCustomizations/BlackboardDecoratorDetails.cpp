// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailCustomizations/BlackboardDecoratorDetails.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SlateOptMacros.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "BehaviorTree/BTNode.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"
#include "BehaviorTree/Decorators/BTDecorator_BlackboardBase.h"
#include "BehaviorTree/BlackboardData.h"
#include "BehaviorTree/Decorators/BTDecorator_Blackboard.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Enum.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_NativeEnum.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Int.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Float.h"

#define LOCTEXT_NAMESPACE "BlackboardDecoratorDetails"

TSharedRef<IDetailCustomization> FBlackboardDecoratorDetails::MakeInstance()
{
	return MakeShareable( new FBlackboardDecoratorDetails );
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FBlackboardDecoratorDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	FBehaviorDecoratorDetails::CustomizeDetails(DetailLayout);

	CacheBlackboardData(DetailLayout);
	const bool bIsEnabled = CachedBlackboardAsset.IsValid();
	TAttribute<bool> PropertyEditCheck = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FBehaviorDecoratorDetails::IsEditingEnabled));

	IDetailCategoryBuilder& FlowCategory = DetailLayout.EditCategory("FlowControl");
	NotifyObserverProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UBTDecorator_Blackboard, NotifyObserver));
	IDetailPropertyRow& AbortRow = FlowCategory.AddProperty(NotifyObserverProperty);
	AbortRow.IsEnabled(PropertyEditCheck);

	IDetailCategoryBuilder& BBCategory = DetailLayout.EditCategory( "Blackboard" );
	IDetailPropertyRow& KeySelectorRow = BBCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UBTDecorator_Blackboard, BlackboardKey)));
	KeySelectorRow.IsEnabled(bIsEnabled);

	KeyIDProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UBTDecorator_Blackboard, BlackboardKey.SelectedKeyID), UBTDecorator_BlackboardBase::StaticClass());
	if (KeyIDProperty.IsValid())
	{
		FSimpleDelegate OnKeyChangedDelegate = FSimpleDelegate::CreateSP( this, &FBlackboardDecoratorDetails::OnKeyIDChanged );
		KeyIDProperty->SetOnPropertyValueChanged(OnKeyChangedDelegate);
		OnKeyIDChanged();
	}

#if WITH_EDITORONLY_DATA

	IDetailPropertyRow& BasicOpRow = BBCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UBTDecorator_Blackboard, BasicOperation)));
	BasicOpRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FBlackboardDecoratorDetails::GetBasicOpVisibility)));
	BasicOpRow.IsEnabled(PropertyEditCheck);
	
	IDetailPropertyRow& ArithmeticOpRow = BBCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UBTDecorator_Blackboard, ArithmeticOperation)));
	ArithmeticOpRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FBlackboardDecoratorDetails::GetArithmeticOpVisibility)));
	ArithmeticOpRow.IsEnabled(PropertyEditCheck);

	IDetailPropertyRow& TextOpRow = BBCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UBTDecorator_Blackboard, TextOperation)));
	TextOpRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FBlackboardDecoratorDetails::GetTextOpVisibility)));
	TextOpRow.IsEnabled(PropertyEditCheck);

#endif // WITH_EDITORONLY_DATA

	IDetailPropertyRow& IntValueRow = BBCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UBTDecorator_Blackboard, IntValue)));
	IntValueRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FBlackboardDecoratorDetails::GetIntValueVisibility)));
	IntValueRow.IsEnabled(PropertyEditCheck);

	IDetailPropertyRow& FloatValueRow = BBCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UBTDecorator_Blackboard, FloatValue)));
	FloatValueRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FBlackboardDecoratorDetails::GetFloatValueVisibility)));
	FloatValueRow.IsEnabled(PropertyEditCheck);

	StringValueProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UBTDecorator_Blackboard, StringValue));
	IDetailPropertyRow& StringValueRow = BBCategory.AddProperty(StringValueProperty);
	StringValueRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FBlackboardDecoratorDetails::GetStringValueVisibility)));
	StringValueRow.IsEnabled(PropertyEditCheck);

	IDetailPropertyRow& EnumValueRow = BBCategory.AddProperty(StringValueProperty);
	EnumValueRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FBlackboardDecoratorDetails::GetEnumValueVisibility)));
	EnumValueRow.IsEnabled(PropertyEditCheck);
	EnumValueRow.CustomWidget()
		.NameContent()
		[
			StringValueProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FBlackboardDecoratorDetails::OnGetEnumValueContent)
 			.ContentPadding(FMargin( 2.0f, 2.0f ))
			.ButtonContent()
			[
				SNew(STextBlock) 
				.Text(this, &FBlackboardDecoratorDetails::GetCurrentEnumValueDesc)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FBlackboardDecoratorDetails::CacheBlackboardData(IDetailLayoutBuilder& DetailLayout)
{
	TArray<TWeakObjectPtr<UObject> > MyOuters;
	DetailLayout.GetObjectsBeingCustomized(MyOuters);

	CachedBlackboardAsset.Reset();
	for (int32 i = 0; i < MyOuters.Num(); i++)
	{
		UBTNode* NodeOb = Cast<UBTNode>(MyOuters[i].Get());
		if (NodeOb)
		{
			CachedBlackboardAsset = NodeOb->GetBlackboardAsset();
			break;
		}
	}
}

void FBlackboardDecoratorDetails::OnKeyIDChanged()
{
	CachedOperationType = EBlackboardKeyOperation::Basic;
	CachedCustomObjectType = nullptr;
	CachedKeyType = nullptr;

	UBlackboardData* Blackboard = CachedBlackboardAsset.Get();
	if (Blackboard == nullptr)
	{
		return;
	}

	int32 KeyID;
	FPropertyAccess::Result Result = KeyIDProperty->GetValue(KeyID);
	if (Result == FPropertyAccess::Success)
	{
		const FBlackboardEntry* KeyEntry = Blackboard->GetKey(KeyID);
		if(KeyEntry && KeyEntry->KeyType)
		{
			CachedKeyType = KeyEntry->KeyType->GetClass();
			CachedOperationType = IntCastChecked<uint8>((int32)KeyEntry->KeyType->GetTestOperation());
		}
	}

	// special handling of enum type: cache all names for combo box (display names)
	UEnum* SelectedEnumType = NULL;
	if (CachedKeyType == UBlackboardKeyType_Enum::StaticClass())
	{
		const FBlackboardEntry* EntryInfo = Blackboard->GetKey(KeyID);
		SelectedEnumType = ((UBlackboardKeyType_Enum*)(EntryInfo->KeyType))->EnumType;
	}
	else if (CachedKeyType == UBlackboardKeyType_NativeEnum::StaticClass())
	{
		const FBlackboardEntry* EntryInfo = Blackboard->GetKey(KeyID);
		SelectedEnumType = ((UBlackboardKeyType_NativeEnum*)(EntryInfo->KeyType))->EnumType;
	}

	if (SelectedEnumType)
	{
		CachedCustomObjectType = SelectedEnumType;
		RefreshEnumPropertyValues();
	}
}

void FBlackboardDecoratorDetails::RefreshEnumPropertyValues()
{
	EnumPropValues.Reset();

	if (CachedCustomObjectType)
	{
		for (int32 i = 0; i < CachedCustomObjectType->NumEnums() - 1; i++)
		{
			FString DisplayedName = CachedCustomObjectType->GetDisplayNameTextByIndex(i).ToString();
			EnumPropValues.Add(DisplayedName);
		}
	}
}

void FBlackboardDecoratorDetails::PreChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType)
{
	// Implementing interface pure virtual method but nothing to do here
}

void FBlackboardDecoratorDetails::PostChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType)
{
	if (Changed != nullptr && CachedCustomObjectType == Changed)
	{
		RefreshEnumPropertyValues();
	}
}

TSharedRef<SWidget> FBlackboardDecoratorDetails::OnGetEnumValueContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);

	for (int32 i = 0; i < EnumPropValues.Num(); i++)
	{
		FUIAction ItemAction( FExecuteAction::CreateSP( const_cast<FBlackboardDecoratorDetails*>(this), &FBlackboardDecoratorDetails::OnEnumValueComboChange, i ) );
		MenuBuilder.AddMenuEntry( FText::FromString( EnumPropValues[i] ), TAttribute<FText>(), FSlateIcon(), ItemAction);
	}

	return MenuBuilder.MakeWidget();
}

FText FBlackboardDecoratorDetails::GetCurrentEnumValueDesc() const
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	int32 EnumIndex = INDEX_NONE;

	if (CachedCustomObjectType)
	{	
		// Always use string value to recompute matching index since enumeration
		// can be modified while Editing (i.e. UserDefinedEnumeration)
		FString CurrentStringValue;
		Result = StringValueProperty->GetValue(CurrentStringValue);
		EnumIndex = CachedCustomObjectType->GetIndexByNameString(CurrentStringValue);
	}

	return (Result == FPropertyAccess::Success && EnumPropValues.IsValidIndex(EnumIndex))
		? FText::FromString(EnumPropValues[EnumIndex])
		: FText::GetEmpty();
}

void FBlackboardDecoratorDetails::OnEnumValueComboChange(int32 Index)
{
	if (CachedCustomObjectType)
	{
		const FString NewStringValue = CachedCustomObjectType->GetNameStringByIndex(Index);
		StringValueProperty->SetValue(NewStringValue);
	}
}

EVisibility FBlackboardDecoratorDetails::GetIntValueVisibility() const
{
	return (CachedKeyType == UBlackboardKeyType_Int::StaticClass()) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FBlackboardDecoratorDetails::GetFloatValueVisibility() const
{
	return (CachedKeyType == UBlackboardKeyType_Float::StaticClass()) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FBlackboardDecoratorDetails::GetStringValueVisibility() const
{
	return (CachedOperationType == EBlackboardKeyOperation::Text) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FBlackboardDecoratorDetails::GetEnumValueVisibility() const
{
	if (CachedKeyType == UBlackboardKeyType_Enum::StaticClass() ||
		CachedKeyType == UBlackboardKeyType_NativeEnum::StaticClass())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility FBlackboardDecoratorDetails::GetBasicOpVisibility() const
{
	return (CachedOperationType == EBlackboardKeyOperation::Basic) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FBlackboardDecoratorDetails::GetArithmeticOpVisibility() const
{
	return (CachedOperationType == EBlackboardKeyOperation::Arithmetic) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FBlackboardDecoratorDetails::GetTextOpVisibility() const
{
	return (CachedOperationType == EBlackboardKeyOperation::Text) ? EVisibility::Visible : EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
