// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "DetailCustomizations/BehaviorDecoratorDetails.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Kismet2/EnumEditorUtils.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailCustomization;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;
class UBlackboardData;
class UEnum;
class UUserDefinedEnum;

class FBlackboardDecoratorDetails : public FBehaviorDecoratorDetails, public FEnumEditorUtils::INotifyOnEnumChanged
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

	/** INotifyOnEnumChanged interface */
	virtual void PreChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType) override;
	virtual void PostChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType) override;

private:

	void CacheBlackboardData(IDetailLayoutBuilder& DetailLayout);
	void RefreshEnumPropertyValues();

	EVisibility GetIntValueVisibility() const;
	EVisibility GetFloatValueVisibility() const;
	EVisibility GetStringValueVisibility() const;
	EVisibility GetEnumValueVisibility() const;
	EVisibility GetBasicOpVisibility() const;
	EVisibility GetArithmeticOpVisibility() const;
	EVisibility GetTextOpVisibility() const;

	void OnKeyIDChanged();

	void OnEnumValueComboChange(int32 Index);
	TSharedRef<SWidget> OnGetEnumValueContent() const;
	FText GetCurrentEnumValueDesc() const;

	TSharedPtr<IPropertyHandle> StringValueProperty;
	TSharedPtr<IPropertyHandle> KeyIDProperty;
	TSharedPtr<IPropertyHandle> NotifyObserverProperty;
	
	/** cached type of property selected by KeyName */
	TSubclassOf<class UBlackboardKeyType> CachedKeyType;
	/** cached custom object type of property selected by KeyName */
	UEnum* CachedCustomObjectType;
	/** cached operation for key type */
	uint8 CachedOperationType;
	
	/** cached enum value if property selected by KeyName has enum type */	
	TArray<FString> EnumPropValues;

	TWeakObjectPtr<UBlackboardData> CachedBlackboardAsset;
};
