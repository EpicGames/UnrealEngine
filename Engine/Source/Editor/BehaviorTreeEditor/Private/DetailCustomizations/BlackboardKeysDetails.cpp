// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackboardKeysDetails.h"

#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "UObject/Field.h"

#include "BehaviorTree/Blackboard/BlackboardKeyType_Class.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Enum.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"
#include "BehaviorTree/BlackboardData.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "SlateOptMacros.h"
#include "SEnumCombo.h"
#include "SlotBase.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "BlackboardKeysDetails"

TSharedRef<IDetailCustomization> FBlackboardKeyDetails_Class::MakeInstance()
{
	return MakeShareable(new FBlackboardKeyDetails_Class());
}

void FBlackboardKeyDetails_Class::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	CachedUtils = DetailBuilder.GetPropertyUtilities();
	BaseClassProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBlackboardKeyType_Class, BaseClass));
	BaseClassProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FBlackboardKeyDetails_Class::OnBaseClassChanged));

	UObject* BaseClassObject = nullptr;
	BaseClassProperty->GetValue(BaseClassObject);
	UClass* BaseClass = Cast<UClass>(BaseClassObject);

	DefaultValueProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBlackboardKeyType_Class, DefaultValue));
	DefaultValueProperty->MarkHiddenByCustomization();

	DetailBuilder.AddCustomRowToCategory(DefaultValueProperty, LOCTEXT("DefaultValue", "DefaultValue"))
		.NameContent()
		[
			DefaultValueProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SClassPropertyEntryBox)
				.MetaClass(BaseClass)
				.AllowNone(true)
				.AllowAbstract(true)
				.OnSetClass(this, &FBlackboardKeyDetails_Class::OnSetClass)
				.SelectedClass(this, &FBlackboardKeyDetails_Class::OnGetSelectedClass)
		];
}

void FBlackboardKeyDetails_Class::OnBaseClassChanged()
{
	UObject* BaseClassObject = nullptr;
	BaseClassProperty->GetValue(BaseClassObject);
	const UClass* BaseClass = Cast<UClass>(BaseClassObject);
	const UClass* SelectedClass = OnGetSelectedClass();
	if (SelectedClass && SelectedClass != BaseClass && !SelectedClass->IsChildOf(BaseClass))
	{
		OnSetClass(nullptr);
	}

	if (CachedUtils)
	{
		CachedUtils->ForceRefresh();
	}
}

void FBlackboardKeyDetails_Class::OnSetClass(const UClass* NewClass)
{
	DefaultValueProperty->SetValue(NewClass);
}

const UClass* FBlackboardKeyDetails_Class::OnGetSelectedClass() const
{
	UObject* DefaultValue = nullptr;
	DefaultValueProperty->GetValue(DefaultValue);
	return Cast<UClass>(DefaultValue);
}

TSharedRef<IDetailCustomization> FBlackboardKeyDetails_Object::MakeInstance()
{
	return MakeShareable(new FBlackboardKeyDetails_Object());
}

void FBlackboardKeyDetails_Object::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	CachedUtils = DetailBuilder.GetPropertyUtilities();
	BaseClassProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBlackboardKeyType_Object, BaseClass));
	BaseClassProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FBlackboardKeyDetails_Object::OnBaseClassChanged));

	UObject* BaseClassObject = nullptr;
	BaseClassProperty->GetValue(BaseClassObject);
	UClass* BaseClass = Cast<UClass>(BaseClassObject);

	DefaultValueProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBlackboardKeyType_Object, DefaultValue));
	DefaultValueProperty->MarkHiddenByCustomization();

	DetailBuilder.AddCustomRowToCategory(DefaultValueProperty, LOCTEXT("DefaultValue", "DefaultValue"))
		.NameContent()
		[
			DefaultValueProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SObjectPropertyEntryBox)
				.AllowedClass(BaseClass)
				.AllowClear(true)
				.OnObjectChanged(this, &FBlackboardKeyDetails_Object::OnObjectChanged)
				.ObjectPath(this, &FBlackboardKeyDetails_Object::OnGetObjectPath)
		];
}

void FBlackboardKeyDetails_Object::OnBaseClassChanged()
{
	UObject* BaseClassObject = nullptr;
	BaseClassProperty->GetValue(BaseClassObject);
	const UClass* BaseClass = Cast<UClass>(BaseClassObject);
	const UObject* SelectedObject = OnGetSelectedObject();
	if (SelectedObject && SelectedObject->GetClass() != BaseClass && !SelectedObject->GetClass()->IsChildOf(BaseClass))
	{
		DefaultValueProperty->SetValue(static_cast<UObject*>(nullptr));
	}

	if (CachedUtils)
	{
		CachedUtils->ForceRefresh();
	}
}

const UObject* FBlackboardKeyDetails_Object::OnGetSelectedObject() const
{
	UObject* DefaultValue = nullptr;
	DefaultValueProperty->GetValue(DefaultValue);
	return DefaultValue;
}

void FBlackboardKeyDetails_Object::OnObjectChanged(const FAssetData& AssetData)
{
	DefaultValueProperty->SetValue(AssetData.GetAsset());
}

FString FBlackboardKeyDetails_Object::OnGetObjectPath() const
{
	const UObject* DefaultValue = OnGetSelectedObject();
	return DefaultValue ? DefaultValue->GetPathName() : FString();
}

TSharedRef<IDetailCustomization> FBlackboardKeyDetails_Enum::MakeInstance()
{
	return MakeShareable(new FBlackboardKeyDetails_Enum());
}

void FBlackboardKeyDetails_Enum::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	CachedUtils = DetailBuilder.GetPropertyUtilities();
	EnumTypeProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBlackboardKeyType_Enum, EnumType));
	EnumTypeProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FBlackboardKeyDetails_Enum::OnEnumTypeChanged));

	UObject* EnumTypeObject = nullptr;
	EnumTypeProperty->GetValue(EnumTypeObject);
	UEnum* EnumType = Cast<UEnum>(EnumTypeObject);

	DefaultValueProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBlackboardKeyType_Enum, DefaultValue));
	DefaultValueProperty->MarkHiddenByCustomization();

	if(EnumType)
	{
		DetailBuilder.AddCustomRowToCategory(DefaultValueProperty, LOCTEXT("DefaultValue", "DefaultValue"))
			.NameContent()
			[
				DefaultValueProperty->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SEnumComboBox, EnumType)
					.CurrentValue_Raw(this, &FBlackboardKeyDetails_Enum::GetEnumValue)
					.OnEnumSelectionChanged(SEnumComboBox::FOnEnumSelectionChanged::CreateSP(this, &FBlackboardKeyDetails_Enum::OnEnumSelectionChanged))
			];
	}
}

void FBlackboardKeyDetails_Enum::OnEnumSelectionChanged(int32 NewValue, ESelectInfo::Type)
{
	DefaultValueProperty->SetValue(static_cast<uint8>(NewValue));
}

void FBlackboardKeyDetails_Enum::OnEnumTypeChanged()
{
	DefaultValueProperty->SetValue(static_cast<uint8>(0));
	if (CachedUtils)
	{
		CachedUtils->ForceRefresh();
	}
}

int32 FBlackboardKeyDetails_Enum::GetEnumValue() const
{
	uint8 EnumValue = 0;
	DefaultValueProperty->GetValue(EnumValue);
	return EnumValue;
}
#undef LOCTEXT_NAMESPACE
