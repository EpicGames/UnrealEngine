// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IPropertyHandle;
class IPropertyUtilities;
class SWidget;
struct FAssetData;

namespace ESelectInfo { enum Type : int; }

class FBlackboardKeyDetails_Class : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	TSharedPtr<IPropertyHandle> BaseClassProperty;
	TSharedPtr<IPropertyHandle> DefaultValueProperty;
	TSharedPtr<IPropertyUtilities> CachedUtils;

	void OnBaseClassChanged();
	void OnSetClass(const UClass* NewClass);
	const UClass* OnGetSelectedClass() const;
};

class FBlackboardKeyDetails_Object : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	TSharedPtr<IPropertyHandle> BaseClassProperty;
	TSharedPtr<IPropertyHandle> DefaultValueProperty;
	TSharedPtr<IPropertyUtilities> CachedUtils;

	void OnBaseClassChanged();
	const UObject* OnGetSelectedObject() const;
	void OnObjectChanged(const FAssetData& AssetData);
	FString OnGetObjectPath() const;
};

class FBlackboardKeyDetails_Enum : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	TSharedPtr<IPropertyHandle> EnumTypeProperty;
	TSharedPtr<IPropertyHandle> DefaultValueProperty;
	TSharedPtr<IPropertyUtilities> CachedUtils;

	void OnEnumSelectionChanged(int32 NewValue, ESelectInfo::Type);
	void OnEnumTypeChanged();
	int32 GetEnumValue() const;
};