// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailCustomizations/BlackboardSelectorDetails.h"

#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "BehaviorTree/BlackboardAssetProvider.h"
#include "BehaviorTree/BlackboardData.h"
#include "BehaviorTreeDebugger.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "IPropertyUtilities.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "BlackboardSelectorDetails"

TSharedRef<IPropertyTypeCustomization> FBlackboardSelectorDetails::MakeInstance()
{
	return MakeShareable( new FBlackboardSelectorDetails );
}

void FBlackboardSelectorDetails::CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	MyStructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities().Get();

	CacheBlackboardData();
	
	HeaderRow.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FBlackboardSelectorDetails::IsEditingEnabled)))
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FBlackboardSelectorDetails::OnGetKeyContent)
 			.ContentPadding(FMargin( 2.0f, 2.0f ))
			.IsEnabled(this, &FBlackboardSelectorDetails::IsEditingEnabled)
			.ButtonContent()
			[
				SNew(STextBlock) 
				.Text(this, &FBlackboardSelectorDetails::GetCurrentKeyDesc)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];

	InitKeyFromProperty();
}

void FBlackboardSelectorDetails::CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
}

void FBlackboardSelectorDetails::FindBlackboardAsset(const UObject* InObj, const UObject*& OutBlackboardOwner, UBlackboardData*& OutBlackboardAsset) const
{
	OutBlackboardOwner = nullptr;
	OutBlackboardAsset = nullptr;

	// Find first blackboard provider and return its' data.
	// The data might be null if no blackboard is set up yet by the provider.
	// It is important to always return the same provider so that the invalidate check
	// in OnBlackboardOwnerChanged works consistently.
	for (const UObject* TestOb = InObj; TestOb; TestOb = TestOb->GetOuter())
	{
		const IBlackboardAssetProvider* Provider = Cast<IBlackboardAssetProvider>(TestOb);
		if (Provider)
		{
			OutBlackboardOwner = TestOb;
			OutBlackboardAsset = Provider->GetBlackboardAsset();
			break;
		}
	}
}

void FBlackboardSelectorDetails::CacheBlackboardData()
{
	TSharedPtr<IPropertyHandleArray> MyFilterProperty = MyStructProperty->GetChildHandle(TEXT("AllowedTypes"))->AsArray();
	MyKeyNameProperty = MyStructProperty->GetChildHandle(TEXT("SelectedKeyName"));
	MyKeyIDProperty = MyStructProperty->GetChildHandle(TEXT("SelectedKeyID"));
	MyKeyClassProperty = MyStructProperty->GetChildHandle(TEXT("SelectedKeyType"));

	TSharedPtr<IPropertyHandle> NonesAllowed = MyStructProperty->GetChildHandle(TEXT("bNoneIsAllowedValue"));
	NonesAllowed->GetValue(bNoneIsAllowedValue);
	
	KeyValues.Reset();

	if (bNoneIsAllowedValue)
	{
		KeyValues.AddUnique(TEXT("None"));
	}

	TArray<UBlackboardKeyType*> FilterObjects;
	
	uint32 NumElements = 0;
	FPropertyAccess::Result Result = MyFilterProperty->GetNumElements(NumElements);
	if (Result == FPropertyAccess::Success)
	{
		for (uint32 Idx = 0; Idx < NumElements; Idx++)
		{
			UObject* FilterOb;
			Result = MyFilterProperty->GetElement(Idx)->GetValue(FilterOb);
			if (Result == FPropertyAccess::Success)
			{
				UBlackboardKeyType* KeyFilterOb = Cast<UBlackboardKeyType>(FilterOb);
				if (KeyFilterOb)
				{
					FilterObjects.Add(KeyFilterOb);
				}
			}
		}
	}

	TArray<UObject*> MyObjects;
	MyStructProperty->GetOuterObjects(MyObjects);
	for (int32 ObjectIdx = 0; ObjectIdx < MyObjects.Num(); ObjectIdx++)
	{
		const UObject* BlackboardOwner = nullptr;
		UBlackboardData* BlackboardAsset = nullptr;
		FindBlackboardAsset(MyObjects[ObjectIdx], BlackboardOwner, BlackboardAsset);

		if (BlackboardAsset)
		{
			CachedBlackboardAssetOwner = BlackboardOwner;
			CachedBlackboardAsset = BlackboardAsset;

			TArray<FName> ProcessedNames;
			for (UBlackboardData* It = BlackboardAsset; It; It = It->Parent)
			{
				for (int32 KeyIdx = 0; KeyIdx < It->Keys.Num(); KeyIdx++)
				{
					const FBlackboardEntry& EntryInfo = It->Keys[KeyIdx];
					bool bIsKeyOverridden = ProcessedNames.Contains(EntryInfo.EntryName);
					bool bIsEntryAllowed = !bIsKeyOverridden && (EntryInfo.KeyType != NULL);

					ProcessedNames.Add(EntryInfo.EntryName);

					if (bIsEntryAllowed && FilterObjects.Num())
					{
						bool bFilterPassed = false;
						for (int32 FilterIdx = 0; FilterIdx < FilterObjects.Num(); FilterIdx++)
						{
							if (EntryInfo.KeyType->IsAllowedByFilter(FilterObjects[FilterIdx]))
							{
								bFilterPassed = true;
								break;
							}
						}

						bIsEntryAllowed = bFilterPassed;
					}

					if (bIsEntryAllowed)
					{
						KeyValues.AddUnique(EntryInfo.EntryName);
					}
				}
			}

			break;
		}
	}

	if (GetDefault<UEditorPerProjectUserSettings>()->bDisplayBlackboardKeysInAlphabeticalOrder)
	{
		KeyValues.Sort([](const FName& a, const FName& b) { return a.LexicalLess(b); });
	}

	if (!OnBlackboardDataChangedHandle.IsValid())
	{
		OnBlackboardDataChangedHandle = UBlackboardData::OnBlackboardDataChanged.AddSP(this, &FBlackboardSelectorDetails::OnBlackboardDataChanged);
	}
	if (!OnBlackboardOwnerChangedHandle.IsValid())
	{
		OnBlackboardOwnerChangedHandle = IBlackboardAssetProvider::OnBlackboardOwnerChanged.AddSP(this, &FBlackboardSelectorDetails::OnBlackboardOwnerChanged);
	}

}

void FBlackboardSelectorDetails::OnBlackboardDataChanged(UBlackboardData* Asset)
{
	UBlackboardData* CachedAsset = CachedBlackboardAsset.Get();
	if (CachedAsset == nullptr || CachedAsset == Asset)
	{
		CacheBlackboardData();
		InitKeyFromProperty();
	}
}

void FBlackboardSelectorDetails::OnBlackboardOwnerChanged(UObject* Owner, UBlackboardData* Asset)
{
	const UObject* CachedAssetOwner = CachedBlackboardAssetOwner.Get();
	if (CachedAssetOwner == nullptr || CachedAssetOwner == Owner)
	{
		CacheBlackboardData();
		InitKeyFromProperty();
	}
}

void FBlackboardSelectorDetails::InitKeyFromProperty()
{
	FName KeyNameValue;
	FPropertyAccess::Result Result = MyKeyNameProperty->GetValue(KeyNameValue);
	if (Result == FPropertyAccess::Success)
	{
		const int32 KeyIdx = KeyValues.IndexOfByKey(KeyNameValue);
		if (KeyIdx == INDEX_NONE)
		{
			if (bNoneIsAllowedValue == false)
			{
				const FName PropName = MyStructProperty->GetProperty() ? MyStructProperty->GetProperty()->GetFName() : NAME_None;
				const int32 KeyNameIdx = KeyValues.IndexOfByKey(PropName);

				OnKeyComboChange(KeyNameIdx != INDEX_NONE ? KeyNameIdx : 0);
			}
			else
			{
				// Set ID first so callbacks can properly test against InvalidKey
				MyKeyIDProperty->SetValue((int32)FBlackboard::InvalidKey);
				MyKeyClassProperty->SetValue((UObject*)nullptr);				
				MyKeyNameProperty->SetValue(TEXT("None"));
			}
		}
	}
}

TSharedRef<SWidget> FBlackboardSelectorDetails::OnGetKeyContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);

	for (int32 Idx = 0; Idx < KeyValues.Num(); Idx++)
	{
		FUIAction ItemAction( FExecuteAction::CreateSP( const_cast<FBlackboardSelectorDetails*>(this), &FBlackboardSelectorDetails::OnKeyComboChange, Idx) );
		MenuBuilder.AddMenuEntry( FText::FromName( KeyValues[Idx] ), TAttribute<FText>(), FSlateIcon(), ItemAction);
	}

	return MenuBuilder.MakeWidget();
}

FText FBlackboardSelectorDetails::GetCurrentKeyDesc() const
{
	FName NameValue;
	MyKeyNameProperty->GetValue(NameValue);

	const int32 KeyIdx = KeyValues.IndexOfByKey(NameValue);
	return KeyValues.IsValidIndex(KeyIdx) ? FText::FromName(KeyValues[KeyIdx]) : FText::FromName(NameValue);
}

void FBlackboardSelectorDetails::OnKeyComboChange(int32 Index)
{
	if (KeyValues.IsValidIndex(Index))
	{
		UBlackboardData* BlackboardAsset = CachedBlackboardAsset.Get();
		if (BlackboardAsset)
		{
			const FBlackboard::FKey KeyID = BlackboardAsset->GetKeyID(KeyValues[Index]);
			const UObject* KeyClass = BlackboardAsset->GetKeyType(KeyID);

			MyKeyClassProperty->SetValue(KeyClass);
			MyKeyIDProperty->SetValue((int32)KeyID);

			MyKeyNameProperty->SetValue(KeyValues[Index]);
		}
	}
}

bool FBlackboardSelectorDetails::IsEditingEnabled() const
{
	return FBehaviorTreeDebugger::IsPIENotSimulating() && PropUtils->IsPropertyEditingEnabled();
}

#undef LOCTEXT_NAMESPACE
