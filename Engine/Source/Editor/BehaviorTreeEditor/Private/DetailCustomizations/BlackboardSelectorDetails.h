// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IPropertyHandle;
class SWidget;
class UBlackboardData;
class UObject;

class FBlackboardSelectorDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

private:

	void CacheBlackboardData();
	void FindBlackboardAsset(const UObject* InObj, const UObject*& OutBlackboardOwner, UBlackboardData*& OutBlackboardAsset) const;

	void OnBlackboardDataChanged(UBlackboardData* Asset);
	void OnBlackboardOwnerChanged(UObject* Owner, UBlackboardData* Asset);

	void InitKeyFromProperty();
	void OnKeyComboChange(int32 Index);
	TSharedRef<SWidget> OnGetKeyContent() const;
	FText GetCurrentKeyDesc() const;
	bool IsEditingEnabled() const;

	FDelegateHandle	OnBlackboardDataChangedHandle;
	FDelegateHandle	OnBlackboardOwnerChangedHandle;

	TSharedPtr<IPropertyHandle> MyStructProperty;
	TSharedPtr<IPropertyHandle> MyKeyNameProperty;
	TSharedPtr<IPropertyHandle> MyKeyIDProperty;
	TSharedPtr<IPropertyHandle> MyKeyClassProperty;

	/** cached names of keys */	
	TArray<FName> KeyValues;

	bool bNoneIsAllowedValue;

	/** cached blackboard asset */
	TWeakObjectPtr<class UBlackboardData> CachedBlackboardAsset;
	TWeakObjectPtr<const class UObject> CachedBlackboardAssetOwner;

	/** property utils */
	class IPropertyUtilities* PropUtils;
};
