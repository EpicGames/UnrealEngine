// Copyright Epic Games, Inc. All Rights Reserved.

#include "ValueOrBBKeyDetails.h"

#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "UObject/Field.h"

#include "BehaviorTree/BTNode.h"
#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTree/BlackboardData.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "SEnumCombo.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ValueOrBBKeyDetails"

TSharedRef<IPropertyTypeCustomization> FValueOrBBKeyDetails::MakeInstance()
{
	return MakeShareable(new FValueOrBBKeyDetails);
}

TSharedRef<IPropertyTypeCustomization> FValueOrBBKeyDetails_Class::MakeInstance()
{
	return MakeShareable(new FValueOrBBKeyDetails_Class);
}

TSharedRef<IPropertyTypeCustomization> FValueOrBBKeyDetails_Enum::MakeInstance()
{
	return MakeShareable(new FValueOrBBKeyDetails_Enum);
}

TSharedRef<IPropertyTypeCustomization> FValueOrBBKeyDetails_Object::MakeInstance()
{
	return MakeShareable(new FValueOrBBKeyDetails_Object);
}

TSharedRef<IPropertyTypeCustomization> FValueOrBBKeyDetails_Struct::MakeInstance()
{
	return MakeShareable(new FValueOrBBKeyDetails_Struct);
}

TSharedRef<IPropertyTypeCustomization> FValueOrBBKeyDetails_WithChild::MakeInstance()
{
	return MakeShareable(new FValueOrBBKeyDetails_WithChild);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FValueOrBBKeyDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	KeyProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FValueOrBlackboardKeyBase, Key));
	DefaultValueProperty = StructPropertyHandle->GetChildHandle(TEXT("DefaultValue"));
	CachedUtils = StructCustomizationUtils.GetPropertyUtilities();

	if (const FProperty* MetaDataProperty = StructPropertyHandle->GetMetaDataProperty())
	{
		if (const TMap<FName, FString>* MetaDataMap = MetaDataProperty->GetMetaDataMap())
		{
			for(const TPair<FName, FString>& MetaData : *MetaDataMap)
			{
				DefaultValueProperty->SetInstanceMetaData(MetaData.Key, MetaData.Value);
			}
		}
	}

	ValidateData();

	TSharedRef<SWidget> DefaultValueWidget = CreateDefaultValueWidget();
	DefaultValueWidget->SetEnabled(TAttribute<bool>(this, &FValueOrBBKeyDetails::CanEditDefaultValue));

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.HAlign(HAlign_Fill)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			DefaultValueWidget
		]
		+ SHorizontalBox::Slot()
		[
			HasAccessToBlackboard() ?
				SNew(SComboButton)
				.OnGetMenuContent(this, &FValueOrBBKeyDetails::OnGetKeyNames)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &FValueOrBBKeyDetails::GetKeyDesc)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				: KeyProperty->CreatePropertyValueWidget()
		]
		.Padding(FMargin(6.0f, 2.0f))
	];
}

void FValueOrBBKeyDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

bool FValueOrBBKeyDetails::CanEditDefaultValue() const
{
	FName KeyValue;
	KeyProperty->GetValue(KeyValue);
	return KeyValue == FName() || !HasAccessToBlackboard();
}

TSharedRef<SWidget> FValueOrBBKeyDetails::CreateDefaultValueWidget()
{
	return DefaultValueProperty->CreatePropertyValueWidget();
}

void FValueOrBBKeyDetails_Class::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if ((BaseClassProperty = StructPropertyHandle->GetChildHandle(TEXT("BaseClass"))))
	{
		BaseClassProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FValueOrBBKeyDetails_Class::OnBaseClassChanged));
	}
	FValueOrBBKeyDetails::CustomizeHeader(StructPropertyHandle, HeaderRow, StructCustomizationUtils);
}

void FValueOrBBKeyDetails_Class::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (BaseClassProperty && BaseClassProperty->IsEditable())
	{
		StructBuilder.AddProperty(BaseClassProperty.ToSharedRef());
	}
}

TSharedRef<SWidget> FValueOrBBKeyDetails_Class::CreateDefaultValueWidget()
{
	if (const FValueOrBlackboardKeyBase* DataPtr = GetDataPtr())
	{
		if (const UClass* BaseClass = static_cast<const FValueOrBBKey_Class*>(DataPtr)->BaseClass)
		{
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				[
					SNew(SClassPropertyEntryBox)
					.MetaClass(BaseClass)
					.AllowNone(true)
					.AllowAbstract(true)
					.OnSetClass(this, &FValueOrBBKeyDetails_Class::OnSetClass)
					.SelectedClass(this, &FValueOrBBKeyDetails_Class::OnGetSelectedClass)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(2.0f, 1.0f)
				[
					PropertyCustomizationHelpers::MakeBrowseButton(FSimpleDelegate::CreateSP(this, &FValueOrBBKeyDetails_Class::BrowseToClass))
				];
		}
	}

	return FValueOrBBKeyDetails::CreateDefaultValueWidget();
}

void FValueOrBBKeyDetails_Enum::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if ((EnumTypeProperty = StructPropertyHandle->GetChildHandle(TEXT("EnumType"))))
	{
		EnumTypeProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FValueOrBBKeyDetails_Enum::OnEnumTypeChanged));
	}

	if ((NativeEnumTypeNameProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FValueOrBBKey_Enum, NativeEnumTypeName))))
	{
		NativeEnumTypeNameProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FValueOrBBKeyDetails_Enum::OnNativeEnumTypeNameChanged));
	}

	FValueOrBBKeyDetails::CustomizeHeader(StructPropertyHandle, HeaderRow, StructCustomizationUtils);
}

void FValueOrBBKeyDetails_Enum::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (EnumTypeProperty && EnumTypeProperty->IsEditable())
	{
		StructBuilder.AddProperty(EnumTypeProperty.ToSharedRef());
	}

	if (NativeEnumTypeNameProperty && NativeEnumTypeNameProperty->IsEditable())
	{
		StructBuilder.AddProperty(NativeEnumTypeNameProperty.ToSharedRef());
	}
}

TSharedRef<SWidget> FValueOrBBKeyDetails_Enum::CreateDefaultValueWidget()
{
	if(const FValueOrBlackboardKeyBase* DataPtr = GetDataPtr())
	{
		if (UEnum* EnumType = static_cast<const FValueOrBBKey_Enum*>(DataPtr)->EnumType)
		{
			return SNew(SEnumComboBox, EnumType)
				.CurrentValue_Raw(this, &FValueOrBBKeyDetails_Enum::GetEnumValue)
				.OnEnumSelectionChanged(SEnumComboBox::FOnEnumSelectionChanged::CreateSP(this, &FValueOrBBKeyDetails_Enum::OnEnumSelectionChanged));
		}
	}
	return FValueOrBBKeyDetails::CreateDefaultValueWidget();
}

void FValueOrBBKeyDetails_Object::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if ((BaseClassProperty = StructPropertyHandle->GetChildHandle(TEXT("BaseClass"))))
	{
		BaseClassProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FValueOrBBKeyDetails_Object::OnBaseClassChanged));
	}

	FValueOrBBKeyDetails::CustomizeHeader(StructPropertyHandle, HeaderRow, StructCustomizationUtils);
}

void FValueOrBBKeyDetails_Object::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (BaseClassProperty && BaseClassProperty->IsEditable())
	{
		StructBuilder.AddProperty(BaseClassProperty.ToSharedRef());
	}
}

TSharedRef<SWidget> FValueOrBBKeyDetails_Object::CreateDefaultValueWidget()
{
	if (const FValueOrBlackboardKeyBase* DataPtr = GetDataPtr())
	{
		if (UClass* BaseClass = static_cast<const FValueOrBBKey_Object*>(DataPtr)->BaseClass)
		{
			return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(SObjectPropertyEntryBox)
						.AllowedClass(BaseClass)
						.AllowClear(true)
						.OnObjectChanged(this, &FValueOrBBKeyDetails_Object::OnObjectChanged)
						.ObjectPath(this, &FValueOrBBKeyDetails_Object::OnGetObjectPath)
					]
					+ SHorizontalBox::Slot()
					 .AutoWidth()
					 .HAlign(HAlign_Center)
					 .VAlign(VAlign_Center)
					 .Padding(2.0f, 1.0f)
					[
						PropertyCustomizationHelpers::MakeBrowseButton(FSimpleDelegate::CreateSP(this, &FValueOrBBKeyDetails_Object::BrowseToObject))
					];
		}
	}
	return FValueOrBBKeyDetails::CreateDefaultValueWidget();
}

void FValueOrBBKeyDetails_Struct::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FValueOrBBKeyDetails::CustomizeHeader(StructPropertyHandle, HeaderRow, StructCustomizationUtils);
	HeaderRow.ShouldAutoExpand(true);

	EditDefaultsOnlyProperty = StructPropertyHandle->GetChildHandle(TEXT("bCanEditDefaultValueType"));
	// We dont have access to the EditDefaultsOnly property or it's not editable. Lock the type so node instance can't change the DefaultValue type.
	if (!EditDefaultsOnlyProperty.IsValid() || !EditDefaultsOnlyProperty->IsEditable())
	{
		DefaultValueProperty->SetInstanceMetaData(TEXT("StructTypeConst"), TEXT(""));
	}
}

void FValueOrBBKeyDetails_Struct::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FValueOrBBKeyDetails::CustomizeChildren(StructPropertyHandle, StructBuilder, StructCustomizationUtils);
	StructBuilder.AddProperty(DefaultValueProperty.ToSharedRef())
	.IsEnabled(TAttribute<bool>(this, &FValueOrBBKeyDetails::CanEditDefaultValue));
}

void FValueOrBBKeyDetails_WithChild::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FValueOrBBKeyDetails::CustomizeChildren(StructPropertyHandle, StructBuilder, StructCustomizationUtils);
	StructBuilder.AddProperty(DefaultValueProperty.ToSharedRef())
	.IsEnabled(TAttribute<bool>(this, &FValueOrBBKeyDetails::CanEditDefaultValue));
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool FValueOrBBKeyDetails::HasAccessToBlackboard() const
{
	TArray<UObject*> OuterObjects;
	KeyProperty->GetOuterObjects(OuterObjects);
	if (OuterObjects.Num() > 0)
	{
		if (const UBTNode* OwnerNode = Cast<UBTNode>(OuterObjects[0]))
		{
			if (const UBehaviorTree* Tree = Cast<UBehaviorTree>(OwnerNode->GetTreeAsset()))
			{
				return Tree->BlackboardAsset != nullptr;
			}
		}
	}
	return false;
}

void FValueOrBBKeyDetails::GetMatchingKeys(TArray<FName>& OutNames)
{
	OutNames.Add(FName());
	TArray<UObject*> OuterObjects;
	KeyProperty->GetOuterObjects(OuterObjects);
	if (OuterObjects.Num())
	{
		if(const FValueOrBlackboardKeyBase* DataPtr = GetDataPtr())
		{
			if (const UBTNode* OwnerNode = Cast<UBTNode>(OuterObjects[0]))
			{
				if (const UBehaviorTree* Tree = Cast<UBehaviorTree>(OwnerNode->GetTreeAsset()))
				{
					const UBlackboardData* Blackboard = Tree->GetBlackboardAsset();
					while (Blackboard)
					{
						for (const FBlackboardEntry& BlackboardEntry : Blackboard->Keys)
						{
							if (DataPtr->IsCompatibleType(BlackboardEntry.KeyType))
							{
								OutNames.Add(BlackboardEntry.EntryName);
							}
						}
						Blackboard = Blackboard->Parent;
					}
				}
			}
		}
	}
}

void FValueOrBBKeyDetails::ValidateData()
{
	if (HasAccessToBlackboard())
	{
		TArray<FName> Keys;
		GetMatchingKeys(Keys);
		FName NameValue;
		KeyProperty->GetValue(NameValue);
		if (!Keys.Contains(NameValue))
		{
			KeyProperty->SetValue(FName());
		}
	}
}

void FValueOrBBKeyDetails_Class::ValidateData()
{
	FValueOrBBKeyDetails::ValidateData();
	UObject* Object = nullptr;
	DefaultValueProperty->GetValue(Object);
	if (UClass* SelectedClass = Cast<UClass>(Object))
	{
		if (const FValueOrBlackboardKeyBase* DataPtr = GetDataPtr())
		{
			const UClass* BaseClass = static_cast<const FValueOrBBKey_Class*>(DataPtr)->BaseClass;
			if (!BaseClass || !SelectedClass->IsChildOf(BaseClass))
			{
				DefaultValueProperty->SetValue(static_cast<UObject*>(nullptr));
			}
		}
	}
}

void FValueOrBBKeyDetails_Enum::ValidateData()
{
	FValueOrBBKeyDetails::ValidateData();
	uint8 Value = 0;
	if (DefaultValueProperty->GetValue(Value) == FPropertyAccess::Success)
	{
		if (const FValueOrBlackboardKeyBase* DataPtr = GetDataPtr())
		{
			const UEnum* EnumType = static_cast<const FValueOrBBKey_Enum*>(DataPtr)->EnumType;
			if (EnumType)
			{
				if (!EnumType->IsValidEnumValue(Value) && EnumType->NumEnums())
				{
					DefaultValueProperty->SetValue(uint8(EnumType->GetValueByIndex(0)));
				}
			}
			else
			{
				DefaultValueProperty->SetValue(uint8(0));
			}
		}
		
	}
}
void FValueOrBBKeyDetails_Object::ValidateData()
{
	FValueOrBBKeyDetails::ValidateData();
	UObject* Object = nullptr;
	DefaultValueProperty->GetValue(Object);
	if (Object)
	{
		if(const FValueOrBlackboardKeyBase* DataPtr = GetDataPtr())
		{
			const UClass* BaseClass = static_cast<const FValueOrBBKey_Object*>(DataPtr)->BaseClass;
			if (!BaseClass || !Object->IsA(BaseClass))
			{
				DefaultValueProperty->SetValue(static_cast<UObject*>(nullptr));
			}
		}
	}
}

void FValueOrBBKeyDetails_Object::OnBaseClassChanged()
{
	ValidateData();
	if (CachedUtils)
	{
		CachedUtils->ForceRefresh();
	}
}

TSharedRef<SWidget> FValueOrBBKeyDetails::OnGetKeyNames()
{
	FMenuBuilder MenuBuilder(true, NULL);
	MatchingKeys.Empty();
	GetMatchingKeys(MatchingKeys);

	for (int32 i = 0; i < MatchingKeys.Num(); i++)
	{
		FUIAction ItemAction(FExecuteAction::CreateSP(this, &FValueOrBBKeyDetails::OnKeyChanged, i));
		MenuBuilder.AddMenuEntry(FText::FromName(MatchingKeys[i]), TAttribute<FText>(), FSlateIcon(), ItemAction);
	}

	return MenuBuilder.MakeWidget();
}

void FValueOrBBKeyDetails::OnKeyChanged(int32 Index)
{
	FName KeyValue = MatchingKeys[Index];
	KeyProperty->SetValue(KeyValue);
}

FText FValueOrBBKeyDetails::GetKeyDesc() const
{
	FName KeyValue;
	KeyProperty->GetValue(KeyValue);

	return FText::FromString(KeyValue.ToString());
}

const FValueOrBlackboardKeyBase* FValueOrBBKeyDetails::GetDataPtr() const
{
	TArray<void*> StructPtrs;
	StructProperty->AccessRawData(StructPtrs);
	return (StructPtrs.Num() == 1) ? reinterpret_cast<FValueOrBlackboardKeyBase*>(StructPtrs[0]) : nullptr;
}

void FValueOrBBKeyDetails_Class::OnBaseClassChanged()
{
	ValidateData();
	if (CachedUtils)
	{
		CachedUtils->ForceRefresh();
	}
}

void FValueOrBBKeyDetails_Class::OnSetClass(const UClass* NewClass)
{
	DefaultValueProperty->SetValue(NewClass);
}

const UClass* FValueOrBBKeyDetails_Class::OnGetSelectedClass() const
{
	UObject* DefaultValue = nullptr;
	DefaultValueProperty->GetValue(DefaultValue);
	return Cast<UClass>(DefaultValue);
}

void FValueOrBBKeyDetails_Class::BrowseToClass() const
{
	UObject* DefaultValue = nullptr;
	DefaultValueProperty->GetValue(DefaultValue);
	if (DefaultValue)
	{
		GEditor->SyncBrowserToObject(DefaultValue);
	}
}

void FValueOrBBKeyDetails_Enum::OnEnumTypeChanged()
{
	ValidateData();
	if (CachedUtils)
	{
		CachedUtils->ForceRefresh();
	}
}

void FValueOrBBKeyDetails_Enum::OnEnumSelectionChanged(int32 NewValue, ESelectInfo::Type)
{
	DefaultValueProperty->SetValue(static_cast<uint8>(NewValue));
}

void FValueOrBBKeyDetails_Enum::OnNativeEnumTypeNameChanged()
{
	EnumTypeProperty->SetValue(static_cast<UEnum*>(nullptr));

	FString NativeEnumTypeName;
	if (NativeEnumTypeNameProperty->GetValue(NativeEnumTypeName) == FPropertyAccess::Success)
	{
		if (const UEnum* const NativeEnumType = UClass::TryFindTypeSlow<UEnum>(NativeEnumTypeName, EFindFirstObjectOptions::ExactClass))
		{
			EnumTypeProperty->SetValue(NativeEnumType);
		}
	}

	ValidateData();
	if (CachedUtils)
	{
		CachedUtils->ForceRefresh();
	}
}

int32 FValueOrBBKeyDetails_Enum::GetEnumValue() const
{
	uint8 EnumValue = 0;
	DefaultValueProperty->GetValue(EnumValue);
	return EnumValue;
}

bool FValueOrBBKeyDetails_Enum::CanEditEnumType() const
{
	FString NativeEnumTypeName;
	const FPropertyAccess::Result NativeEnumTypeNamePropertyAccessResult = NativeEnumTypeNameProperty->GetValue(NativeEnumTypeName);
	return (NativeEnumTypeNamePropertyAccessResult != FPropertyAccess::Success) || NativeEnumTypeName.IsEmpty();
}

void FValueOrBBKeyDetails_Object::OnObjectChanged(const FAssetData& AssetData)
{
	DefaultValueProperty->SetValue(AssetData.GetAsset());
}

FString FValueOrBBKeyDetails_Object::OnGetObjectPath() const
{
	UObject* DefaultValue = nullptr;
	DefaultValueProperty->GetValue(DefaultValue);
	return DefaultValue ? DefaultValue->GetPathName() : FString();
}

void FValueOrBBKeyDetails_Object::BrowseToObject() const
{
	UObject* DefaultValue = nullptr;
	DefaultValueProperty->GetValue(DefaultValue);
	if (DefaultValue)
	{
		GEditor->SyncBrowserToObject(DefaultValue);
	}
}
#undef LOCTEXT_NAMESPACE