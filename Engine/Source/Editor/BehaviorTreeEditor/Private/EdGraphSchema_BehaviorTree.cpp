// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraphSchema_BehaviorTree.h"

#include "AIGraphTypes.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/BTService.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/Composites/BTComposite_SimpleParallel.h"
#include "BehaviorTree/Tasks/BTTask_RunBehavior.h"
#include "BehaviorTreeConnectionDrawingPolicy.h"
#include "BehaviorTreeDebugger.h"
#include "BehaviorTreeEditorModule.h"
#include "BehaviorTreeEditorTypes.h"
#include "BehaviorTreeGraph.h"
#include "BehaviorTreeGraphNode.h"
#include "BehaviorTreeGraphNode_Composite.h"
#include "BehaviorTreeGraphNode_CompositeDecorator.h"
#include "BehaviorTreeGraphNode_Decorator.h"
#include "BehaviorTreeGraphNode_Root.h"
#include "BehaviorTreeGraphNode_Service.h"
#include "BehaviorTreeGraphNode_SimpleParallel.h"
#include "BehaviorTreeGraphNode_SubtreeTask.h"
#include "BehaviorTreeGraphNode_Task.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "GraphEditorActions.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Modules/ModuleManager.h"
#include "ObjectEditorUtils.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphSchema_BehaviorTree)

#define LOCTEXT_NAMESPACE "BehaviorTreeEditor"

int32 UEdGraphSchema_BehaviorTree::CurrentCacheRefreshID = 0;

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
UEdGraphNode* FBehaviorTreeSchemaAction_AutoArrange::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UBehaviorTreeGraph* Graph = Cast<UBehaviorTreeGraph>(ParentGraph);
	if (Graph)
	{
		Graph->AutoArrange();
	}

	return NULL;
}

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//

UEdGraphSchema_BehaviorTree::UEdGraphSchema_BehaviorTree(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	CompositeDecoratorClass = UBehaviorTreeGraphNode_CompositeDecorator::StaticClass();
	DecoratorClass = UBehaviorTreeGraphNode_Decorator::StaticClass();
	TaskClass = UBehaviorTreeGraphNode_Task::StaticClass();
	ServiceClass = UBehaviorTreeGraphNode_Service::StaticClass();
}

void UEdGraphSchema_BehaviorTree::CreateDefaultNodesForGraph(UEdGraph& Graph) const
{
	FGraphNodeCreator<UBehaviorTreeGraphNode_Root> NodeCreator(Graph);
	UBehaviorTreeGraphNode_Root* MyNode = NodeCreator.CreateNode(true, Cast<UBehaviorTreeGraph>(&Graph)->RootNodeClass);
	NodeCreator.Finalize();
	SetNodeMetaData(MyNode, FNodeMetadata::DefaultGraphNode);
}

void UEdGraphSchema_BehaviorTree::GetGraphNodeContextActions(FGraphContextMenuBuilder& ContextMenuBuilder, int32 SubNodeFlags) const
{
	Super::GetGraphNodeContextActions(ContextMenuBuilder, SubNodeFlags);

	if (SubNodeFlags == ESubNode::Decorator)
	{
		const FText Category = FObjectEditorUtils::GetCategoryText(CompositeDecoratorClass);
		UEdGraph* Graph = (UEdGraph*)ContextMenuBuilder.CurrentGraph;
		UBehaviorTreeGraphNode_CompositeDecorator* OpNode = NewObject<UBehaviorTreeGraphNode_CompositeDecorator>(Graph, CompositeDecoratorClass);
		TSharedPtr<FAISchemaAction_NewSubNode> AddOpAction = UAIGraphSchema::AddNewSubNodeAction(ContextMenuBuilder, Category, FText::FromString(OpNode->GetNodeTypeDescription()), OpNode->GetTooltipText());
		AddOpAction->ParentNode = Cast<UBehaviorTreeGraphNode>(ContextMenuBuilder.SelectedObjects[0]);
		AddOpAction->NodeTemplate = OpNode;
	}
}

void UEdGraphSchema_BehaviorTree::GetSubNodeClasses(int32 SubNodeFlags, TArray<FGraphNodeClassData>& ClassData, UClass*& GraphNodeClass) const
{
	FGraphNodeClassHelper& ClassCache = GetClassCache();

	if (SubNodeFlags == ESubNode::Decorator)
	{
		ClassCache.GatherClasses(UBTDecorator::StaticClass(), ClassData);
		GraphNodeClass = DecoratorClass;
	}
	else
	{
		ClassCache.GatherClasses(UBTService::StaticClass(), ClassData);
		GraphNodeClass = ServiceClass;
	}
}

FGraphNodeClassHelper& UEdGraphSchema_BehaviorTree::GetClassCache() const
{
	const FBehaviorTreeEditorModule& EditorModule = FModuleManager::GetModuleChecked<FBehaviorTreeEditorModule>(TEXT("BehaviorTreeEditor"));
	FGraphNodeClassHelper* ClassHelper = EditorModule.GetClassCache().Get();
	check(ClassHelper);
	return *ClassHelper;
}

bool UEdGraphSchema_BehaviorTree::IsNodeSubtreeTask(const FGraphNodeClassData& NodeClass) const
{
	return NodeClass.GetClassName() == UBTTask_RunBehavior::StaticClass()->GetName();
}

void UEdGraphSchema_BehaviorTree::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	const FName PinCategory = ContextMenuBuilder.FromPin ?
		ContextMenuBuilder.FromPin->PinType.PinCategory : 
		UBehaviorTreeEditorTypes::PinCategory_MultipleNodes;

	const bool bNoParent = (ContextMenuBuilder.FromPin == NULL);
	const bool bOnlyTasks = (PinCategory == UBehaviorTreeEditorTypes::PinCategory_SingleTask);
	const bool bOnlyComposites = (PinCategory == UBehaviorTreeEditorTypes::PinCategory_SingleComposite);
	const bool bAllowComposites = bNoParent || !bOnlyTasks || bOnlyComposites;
	const bool bAllowTasks = bNoParent || !bOnlyComposites || bOnlyTasks;

	FGraphNodeClassHelper& ClassCache = GetClassCache();

	if (bAllowComposites)
	{
		FCategorizedGraphActionListBuilder CompositesBuilder(TEXT("Composites"));

		TArray<FGraphNodeClassData> NodeClasses;
		ClassCache.GatherClasses(UBTCompositeNode::StaticClass(), NodeClasses);

		const FString ParallelClassName = UBTComposite_SimpleParallel::StaticClass()->GetName();
		for (const auto& NodeClass : NodeClasses)
		{
			const FText NodeTypeName = FText::FromString(FName::NameToDisplayString(NodeClass.ToString(), false));

			TSharedPtr<FAISchemaAction_NewNode> AddOpAction = UAIGraphSchema::AddNewNodeAction(CompositesBuilder, NodeClass.GetCategory(), NodeTypeName, NodeClass.GetTooltip());

			UClass* GraphNodeClass = UBehaviorTreeGraphNode_Composite::StaticClass();
			if (NodeClass.GetClassName() == ParallelClassName)
			{
				GraphNodeClass = UBehaviorTreeGraphNode_SimpleParallel::StaticClass();
			}

			UBehaviorTreeGraphNode* OpNode = NewObject<UBehaviorTreeGraphNode>(ContextMenuBuilder.OwnerOfTemporaries, GraphNodeClass);
			OpNode->ClassData = NodeClass;
			AddOpAction->NodeTemplate = OpNode;
		}

		ContextMenuBuilder.Append(CompositesBuilder);
	}

	if (bAllowTasks)
	{
		FCategorizedGraphActionListBuilder TasksBuilder(TEXT("Tasks"));

		TArray<FGraphNodeClassData> NodeClasses;
		ClassCache.GatherClasses(UBTTaskNode::StaticClass(), NodeClasses);

		for (const auto& NodeClass : NodeClasses)
		{
			const FText NodeTypeName = FText::FromString(FName::NameToDisplayString(NodeClass.ToString(), false));

			TSharedPtr<FAISchemaAction_NewNode> AddOpAction = UAIGraphSchema::AddNewNodeAction(TasksBuilder, NodeClass.GetCategory(), NodeTypeName, NodeClass.GetTooltip());
			
			UClass* GraphNodeClass = TaskClass;
			if (IsNodeSubtreeTask(NodeClass))
			{
				GraphNodeClass = UBehaviorTreeGraphNode_SubtreeTask::StaticClass();
			}

			UBehaviorTreeGraphNode* OpNode = NewObject<UBehaviorTreeGraphNode>(ContextMenuBuilder.OwnerOfTemporaries, GraphNodeClass);
			OpNode->ClassData = NodeClass;
			AddOpAction->NodeTemplate = OpNode;
		}

		ContextMenuBuilder.Append(TasksBuilder);
	}
	
	if (bNoParent)
	{
		TSharedPtr<FBehaviorTreeSchemaAction_AutoArrange> Action = TSharedPtr<FBehaviorTreeSchemaAction_AutoArrange>(
			new FBehaviorTreeSchemaAction_AutoArrange(FText::GetEmpty(), LOCTEXT("AutoArrange", "Auto Arrange"), FText::GetEmpty(), 0)
			);

		ContextMenuBuilder.AddAction(Action);
	}

	// Add the ability to create a comment to the context menu too for discoverability
	{
		TSharedPtr<FAISchemaAction_AddComment> Action = TSharedPtr<FAISchemaAction_AddComment>(
			new FAISchemaAction_AddComment(LOCTEXT("AddComment", "Add Comment"), LOCTEXT("AddComment_Tooltip", "Adds a comment node to the graph."))
		);

		ContextMenuBuilder.AddAction(Action);
	}
}

void UEdGraphSchema_BehaviorTree::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (Context->Node && !Context->Pin)
	{
		const UBehaviorTreeGraphNode* BTGraphNode = Cast<const UBehaviorTreeGraphNode>(Context->Node);
		if (BTGraphNode && BTGraphNode->CanPlaceBreakpoints())
		{
			FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaBreakpoints", LOCTEXT("BreakpointsHeader", "Breakpoints"));
			{
				Section.AddMenuEntry(FGraphEditorCommands::Get().ToggleBreakpoint);
				Section.AddMenuEntry(FGraphEditorCommands::Get().AddBreakpoint);
				Section.AddMenuEntry(FGraphEditorCommands::Get().RemoveBreakpoint);
				Section.AddMenuEntry(FGraphEditorCommands::Get().EnableBreakpoint);
				Section.AddMenuEntry(FGraphEditorCommands::Get().DisableBreakpoint);
			}
		}
	}

	Super::GetContextMenuActions(Menu, Context);
}

const FPinConnectionResponse UEdGraphSchema_BehaviorTree::CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	// Make sure the pins are not on the same node
	if (PinA->GetOwningNode() == PinB->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("PinErrorSameNode","Both are on the same node"));
	}

	const bool bPinAIsSingleComposite = (PinA->PinType.PinCategory == UBehaviorTreeEditorTypes::PinCategory_SingleComposite);
	const bool bPinAIsSingleTask = (PinA->PinType.PinCategory == UBehaviorTreeEditorTypes::PinCategory_SingleTask);
	const bool bPinAIsSingleNode = (PinA->PinType.PinCategory == UBehaviorTreeEditorTypes::PinCategory_SingleNode);

	const bool bPinBIsSingleComposite = (PinB->PinType.PinCategory == UBehaviorTreeEditorTypes::PinCategory_SingleComposite);
	const bool bPinBIsSingleTask = (PinB->PinType.PinCategory == UBehaviorTreeEditorTypes::PinCategory_SingleTask);
	const bool bPinBIsSingleNode = (PinB->PinType.PinCategory == UBehaviorTreeEditorTypes::PinCategory_SingleNode);

	const bool bPinAIsTask = PinA->GetOwningNode()->IsA(UBehaviorTreeGraphNode_Task::StaticClass());
	const bool bPinAIsComposite = PinA->GetOwningNode()->IsA(UBehaviorTreeGraphNode_Composite::StaticClass());
	
	const bool bPinBIsTask = PinB->GetOwningNode()->IsA(UBehaviorTreeGraphNode_Task::StaticClass());
	const bool bPinBIsComposite = PinB->GetOwningNode()->IsA(UBehaviorTreeGraphNode_Composite::StaticClass());

	if ((bPinAIsSingleComposite && !bPinBIsComposite) || (bPinBIsSingleComposite && !bPinAIsComposite))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("PinErrorOnlyComposite","Only composite nodes are allowed"));
	}

	if ((bPinAIsSingleTask && !bPinBIsTask) || (bPinBIsSingleTask && !bPinAIsTask))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("PinErrorOnlyTask","Only task nodes are allowed"));
	}

	// Compare the directions
	if ((PinA->Direction == EGPD_Input) && (PinB->Direction == EGPD_Input))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("PinErrorInput", "Can't connect input node to input node"));
	}
	else if ((PinB->Direction == EGPD_Output) && (PinA->Direction == EGPD_Output))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("PinErrorOutput", "Can't connect output node to output node"));
	}

	class FNodeVisitorCycleChecker
	{
	public:
		/** Check whether a loop in the graph would be caused by linking the passed-in nodes */
		bool CheckForLoop(UEdGraphNode* StartNode, UEdGraphNode* EndNode)
		{
			VisitedNodes.Add(EndNode);
			return TraverseInputNodesToRoot(StartNode);
		}

	private:
		/** 
		 * Helper function for CheckForLoop()
		 * @param	Node	The node to start traversal at
		 * @return true if we reached a root node (i.e. a node with no input pins), false if we encounter a node we have already seen
		 */
		bool TraverseInputNodesToRoot(UEdGraphNode* Node)
		{
			VisitedNodes.Add(Node);

			// Follow every input pin until we cant any more ('root') or we reach a node we have seen (cycle)
			for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
			{
				UEdGraphPin* MyPin = Node->Pins[PinIndex];

				if (MyPin->Direction == EGPD_Input)
				{
					for (int32 LinkedPinIndex = 0; LinkedPinIndex < MyPin->LinkedTo.Num(); ++LinkedPinIndex)
					{
						UEdGraphPin* OtherPin = MyPin->LinkedTo[LinkedPinIndex];
						if( OtherPin )
						{
							UEdGraphNode* OtherNode = OtherPin->GetOwningNode();
							if (VisitedNodes.Contains(OtherNode))
							{
								return false;
							}
							else
							{
								return TraverseInputNodesToRoot(OtherNode);
							}
						}
					}
				}
			}

			return true;
		}

		TSet<UEdGraphNode*> VisitedNodes;
	};

	// check for cycles
	FNodeVisitorCycleChecker CycleChecker;
	if(!CycleChecker.CheckForLoop(PinA->GetOwningNode(), PinB->GetOwningNode()))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("PinErrorcycle", "Can't create a graph cycle"));
	}

	const bool bPinASingleLink = bPinAIsSingleComposite || bPinAIsSingleTask || bPinAIsSingleNode;
	const bool bPinBSingleLink = bPinBIsSingleComposite || bPinBIsSingleTask || bPinBIsSingleNode;

	if (PinB->Direction == EGPD_Input && PinB->LinkedTo.Num() > 0)
	{
		if(bPinASingleLink)
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_AB, LOCTEXT("PinConnectReplace", "Replace connection"));
		}
		else
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_B, LOCTEXT("PinConnectReplace", "Replace connection"));
		}
	}
	else if (PinA->Direction == EGPD_Input && PinA->LinkedTo.Num() > 0)
	{
		if (bPinBSingleLink)
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_AB, LOCTEXT("PinConnectReplace", "Replace connection"));
		}
		else
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_A, LOCTEXT("PinConnectReplace", "Replace connection"));
		}
	}

	if (bPinASingleLink && PinA->LinkedTo.Num() > 0)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_A, LOCTEXT("PinConnectReplace", "Replace connection"));
	}
	else if(bPinBSingleLink && PinB->LinkedTo.Num() > 0)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_B, LOCTEXT("PinConnectReplace", "Replace connection"));
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, LOCTEXT("PinConnect", "Connect nodes"));
}

const FPinConnectionResponse UEdGraphSchema_BehaviorTree::CanMergeNodes(const UEdGraphNode* NodeA, const UEdGraphNode* NodeB) const
{
	// Make sure the nodes are not the same 
	if (NodeA == NodeB)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Both are the same node"));
	}

	const bool bNodeAIsDecorator = NodeA->IsA(UBehaviorTreeGraphNode_Decorator::StaticClass()) || NodeA->IsA(UBehaviorTreeGraphNode_CompositeDecorator::StaticClass());
	const bool bNodeAIsService = NodeA->IsA(UBehaviorTreeGraphNode_Service::StaticClass());
	const bool bNodeBIsComposite = NodeB->IsA(UBehaviorTreeGraphNode_Composite::StaticClass());
	const bool bNodeBIsTask = NodeB->IsA(UBehaviorTreeGraphNode_Task::StaticClass());
	const bool bNodeBIsDecorator = NodeB->IsA(UBehaviorTreeGraphNode_Decorator::StaticClass()) || NodeB->IsA(UBehaviorTreeGraphNode_CompositeDecorator::StaticClass());
	const bool bNodeBIsService = NodeB->IsA(UBehaviorTreeGraphNode_Service::StaticClass());

	if (FBehaviorTreeDebugger::IsPIENotSimulating())
	{
		if (bNodeAIsDecorator)
		{
			const UBehaviorTreeGraphNode* BTNodeA = Cast<const UBehaviorTreeGraphNode>(NodeA);
			const UBehaviorTreeGraphNode* BTNodeB = Cast<const UBehaviorTreeGraphNode>(NodeB);
			
			if (BTNodeA && BTNodeA->bInjectedNode)
			{
				return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("MergeInjectedNodeNoMove", "Can't move injected nodes!"));
			}

			if (BTNodeB && BTNodeB->bInjectedNode)
			{
				const UBehaviorTreeGraphNode* ParentNodeB = Cast<const UBehaviorTreeGraphNode>(BTNodeB->ParentNode);

				int32 FirstInjectedIdx = INDEX_NONE;
				for (int32 Idx = 0; Idx < ParentNodeB->Decorators.Num(); Idx++)
				{
					if (ParentNodeB->Decorators[Idx]->bInjectedNode)
					{
						FirstInjectedIdx = Idx;
						break;
					}
				}

				int32 NodeIdx = ParentNodeB->Decorators.IndexOfByKey(BTNodeB);
				if (NodeIdx != FirstInjectedIdx)
				{
					return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("MergeInjectedNodeAtEnd", "Decorators must be placed above injected nodes!"));
				}
			}

			if (BTNodeB && BTNodeB->Decorators.Num())
			{
				for (int32 Idx = 0; Idx < BTNodeB->Decorators.Num(); Idx++)
				{
					if (BTNodeB->Decorators[Idx]->bInjectedNode)
					{
						return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("MergeInjectedNodeAtEnd", "Decorators must be placed above injected nodes!"));
					}
				}
			}
		}

		if ((bNodeAIsDecorator && (bNodeBIsComposite || bNodeBIsTask || bNodeBIsDecorator))
			|| (bNodeAIsService && (bNodeBIsComposite || bNodeBIsTask || bNodeBIsService)))
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
		}
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT(""));
}

FLinearColor UEdGraphSchema_BehaviorTree::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	return FColor::White;
}

class FConnectionDrawingPolicy* UEdGraphSchema_BehaviorTree::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const
{
	return new FBehaviorTreeConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements, InGraphObj);
}

bool UEdGraphSchema_BehaviorTree::IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const
{
	return CurrentCacheRefreshID != InVisualizationCacheID;
}

int32 UEdGraphSchema_BehaviorTree::GetCurrentVisualizationCacheID() const
{
	return CurrentCacheRefreshID;
}

void UEdGraphSchema_BehaviorTree::ForceVisualizationCacheClear() const
{
	++CurrentCacheRefreshID;
}

#undef LOCTEXT_NAMESPACE
