// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraphSchema_BehaviorTreeDecorator.h"
#include "Modules/ModuleManager.h"
#include "EdGraph/EdGraph.h"
#include "SoundClassGraph/SoundClassGraphSchema.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTreeDecoratorGraphNode.h"
#include "AIGraphTypes.h"
#include "BehaviorTreeDecoratorGraphNode_Decorator.h"
#include "BehaviorTreeDecoratorGraphNode_Logic.h"
#include "Settings/EditorStyleSettings.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "BehaviorTreeEditorModule.h"
#include "GraphEditorSettings.h"
#include "GraphEditorActions.h"
#include "ScopedTransaction.h"
#include "Framework/Commands/GenericCommands.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphSchema_BehaviorTreeDecorator)

#define LOCTEXT_NAMESPACE "BehaviorTreeDecoratorSchema"

namespace DecoratorSchema
{
	// Maximum distance a drag can be off a node edge to require 'push off' from node
	const int32 NodeDistance = 60;
}

UEdGraphNode* FDecoratorSchemaAction_NewNode::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UEdGraphNode* ResultNode = NULL;

	// If there is a template, we actually use it
	if (NodeTemplate != NULL)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddNode", "Add Node"));
		ParentGraph->Modify();
		if (FromPin)
		{
			FromPin->Modify();
		}

		NodeTemplate->SetFlags(RF_Transactional);

		// set outer to be the graph so it doesn't go away
		NodeTemplate->Rename(NULL, ParentGraph, REN_NonTransactional);

		ParentGraph->AddNode(NodeTemplate, true);

		NodeTemplate->CreateNewGuid();
		NodeTemplate->PostPlacedNewNode();
		NodeTemplate->AllocateDefaultPins();
		NodeTemplate->AutowireNewNode(FromPin);

		// For input pins, new node will generally overlap node being dragged off
		// Work out if we want to visually push away from connected node
		int32 XLocation = Location.X;
		if (FromPin && FromPin->Direction == EGPD_Input)
		{
			UEdGraphNode* PinNode = FromPin->GetOwningNode();
			const float XDelta = FMath::Abs(PinNode->NodePosX - Location.X);

			if (XDelta < DecoratorSchema::NodeDistance)
			{
				// Set location to edge of current node minus the max move distance
				// to force node to push off from connect node enough to give selection handle
				XLocation = PinNode->NodePosX - DecoratorSchema::NodeDistance;
			}
		}

		NodeTemplate->NodePosX = XLocation;
		NodeTemplate->NodePosY = Location.Y;
		NodeTemplate->SnapToGrid(GetDefault<UEditorStyleSettings>()->GridSnapSize);

		ResultNode = NodeTemplate;
	}

	return ResultNode;
}

UEdGraphNode* FDecoratorSchemaAction_NewNode::PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode) 
{
	UEdGraphNode* ResultNode = NULL;
	if (FromPins.Num() > 0)
	{
		ResultNode = PerformAction(ParentGraph, FromPins[0], Location);

		// Try autowiring the rest of the pins
		for (int32 Index = 1; Index < FromPins.Num(); ++Index)
		{
			ResultNode->AutowireNewNode(FromPins[Index]);
		}
	}
	else
	{
		ResultNode = PerformAction(ParentGraph, NULL, Location, bSelectNewNode);
	}

	return ResultNode;
}

void FDecoratorSchemaAction_NewNode::AddReferencedObjects( FReferenceCollector& Collector )
{
	FEdGraphSchemaAction::AddReferencedObjects( Collector );

	// These don't get saved to disk, but we want to make sure the objects don't get GC'd while the action array is around
	Collector.AddReferencedObject( NodeTemplate );
}

//////////////////////////////////////////////////////////////////////////

UEdGraphSchema_BehaviorTreeDecorator::UEdGraphSchema_BehaviorTreeDecorator(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	PC_Boolean = TEXT("bool");
}

TSharedPtr<FDecoratorSchemaAction_NewNode> UEdGraphSchema_BehaviorTreeDecorator::AddNewDecoratorAction(FGraphContextMenuBuilder& ContextMenuBuilder, const FText& Category, const FText& MenuDesc, const FText& Tooltip)
{
	TSharedPtr<FDecoratorSchemaAction_NewNode> NewAction = TSharedPtr<FDecoratorSchemaAction_NewNode>(new FDecoratorSchemaAction_NewNode(Category, MenuDesc, Tooltip, 0));

	ContextMenuBuilder.AddAction( NewAction );

	return NewAction;
}

void UEdGraphSchema_BehaviorTreeDecorator::CreateDefaultNodesForGraph(UEdGraph& Graph) const
{
	FGraphNodeCreator<UBehaviorTreeDecoratorGraphNode_Logic> NodeCreator(Graph);
	UBehaviorTreeDecoratorGraphNode_Logic* MyNode = NodeCreator.CreateNode();
	MyNode->LogicMode = EDecoratorLogicMode::Sink;
	SetNodeMetaData(MyNode, FNodeMetadata::DefaultGraphNode);
	NodeCreator.Finalize();
}

FGraphNodeClassHelper& UEdGraphSchema_BehaviorTreeDecorator::GetClassCache() const
{
	const FBehaviorTreeEditorModule& EditorModule = FModuleManager::GetModuleChecked<FBehaviorTreeEditorModule>(TEXT("BehaviorTreeEditor"));
	FGraphNodeClassHelper* ClassHelper = EditorModule.GetClassCache().Get();
	check(ClassHelper);
	return *ClassHelper;
}

void UEdGraphSchema_BehaviorTreeDecorator::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	const UBehaviorTreeDecoratorGraphNode* ParentGraphNode = ContextMenuBuilder.FromPin 
		? Cast<UBehaviorTreeDecoratorGraphNode>(ContextMenuBuilder.FromPin->GetOuter()) : NULL;

	FGraphNodeClassHelper& ClassCache = GetClassCache();

	TArray<FGraphNodeClassData> NodeClasses;
	ClassCache.GatherClasses(UBTDecorator::StaticClass(), NodeClasses);

	for (const auto& NodeClass : NodeClasses)
	{
		const FText NodeTypeName = FText::FromString(NodeClass.ToString());
		TSharedPtr<FDecoratorSchemaAction_NewNode> AddOpAction = AddNewDecoratorAction(ContextMenuBuilder, NodeClass.GetCategory(), NodeTypeName, FText::GetEmpty());

		UBehaviorTreeDecoratorGraphNode_Decorator* OpNode = NewObject<UBehaviorTreeDecoratorGraphNode_Decorator>(ContextMenuBuilder.OwnerOfTemporaries);
		OpNode->ClassData = NodeClass;
		AddOpAction->NodeTemplate = OpNode;
	}

#if WITH_EDITOR
	UBehaviorTreeDecoratorGraphNode_Logic* LogicCDO = UBehaviorTreeDecoratorGraphNode_Logic::StaticClass()->GetDefaultObject<UBehaviorTreeDecoratorGraphNode_Logic>();
	LogicCDO->GetMenuEntries(ContextMenuBuilder);
#endif
}

void UEdGraphSchema_BehaviorTreeDecorator::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	const UBehaviorTreeDecoratorGraphNode_Logic* LogicNode = Cast<const UBehaviorTreeDecoratorGraphNode_Logic>(Context->Node);
	if (Context->Graph && !Context->Graph->bEditable)
	{
		return;
	}

	if (Context->Node)
	{
		{
			FToolMenuSection& Section = Menu->AddSection("DecoratorGraphSchemaNodeActions", LOCTEXT("ClassActionsMenuHeader", "Node Actions"));
			Section.AddMenuEntry(FGraphEditorCommands::Get().AddExecutionPin);
			Section.AddMenuEntry(FGraphEditorCommands::Get().BreakNodeLinks);
			Section.AddMenuEntry(FGenericCommands::Get().Delete);
		}
	}
}

const FPinConnectionResponse UEdGraphSchema_BehaviorTreeDecorator::CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	if (PinA == NULL || PinB == NULL)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("One of the Pins is NULL"));
	}

	// Make sure the pins are not on the same node
	if (PinA->GetOwningNode() == PinB->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Both are on the same node"));
	}

	// multiple links
	if (PinA->LinkedTo.Num() > 0 || PinB->LinkedTo.Num() > 0)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT(""));
	}

	// Compare the directions
	if (PinA->Direction == PinB->Direction)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT(""));
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
}

FLinearColor UEdGraphSchema_BehaviorTreeDecorator::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	return GetDefault<UGraphEditorSettings>()->BooleanPinTypeColor;
}

bool UEdGraphSchema_BehaviorTreeDecorator::ShouldHidePinDefaultValue(UEdGraphPin* Pin) const
{
	return true;
}

bool UEdGraphSchema_BehaviorTreeDecorator::IsCacheVisualizationOutOfDate(const int32 InVisualizationCacheID) const
{
	return CurrentCacheRefreshID != InVisualizationCacheID;
}

int32 UEdGraphSchema_BehaviorTreeDecorator::GetCurrentVisualizationCacheID() const
{
	return CurrentCacheRefreshID;
}

void UEdGraphSchema_BehaviorTreeDecorator::ForceVisualizationCacheClear() const
{
	++CurrentCacheRefreshID;
}

#undef LOCTEXT_NAMESPACE
