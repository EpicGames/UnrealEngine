// Copyright Epic Games, Inc. All Rights Reserved.
#include "FindInBT.h"

#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTreeEditor.h"
#include "BehaviorTreeGraphNode.h"
#include "BehaviorTreeGraphNode_Decorator.h"
#include "BehaviorTreeGraphNode_Service.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Views/ITypedTableView.h"
#include "GameplayTagContainer.h"
#include "GraphEditor.h"
#include "HAL/PlatformMath.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/WidgetPath.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "FindInBT"

//////////////////////////////////////////////////////////////////////////
// FFindInBTResult

FFindInBTResult::FFindInBTResult(const FString& InValue)
	: Value(InValue), GraphNode(NULL)
{
}

FFindInBTResult::FFindInBTResult(const FString& InValue, TSharedPtr<FFindInBTResult>& InParent, UEdGraphNode* InNode)
	: Value(InValue), GraphNode(InNode), Parent(InParent)
{
}

void FFindInBTResult::SetNodeHighlight(bool bHighlight)
{
	if (GraphNode.IsValid())
	{
		UBehaviorTreeGraphNode* BTNode = Cast<UBehaviorTreeGraphNode>(GraphNode.Get());
		if (BTNode)
		{
			BTNode->bHighlightInSearchTree = bHighlight;
		}
	}
}

TSharedRef<SWidget> FFindInBTResult::CreateIcon() const
{
	FSlateColor IconColor = FSlateColor::UseForeground();
	const FSlateBrush* Brush = NULL;

	if (GraphNode.IsValid())
	{
		if (Cast<UBehaviorTreeGraphNode_Service>(GraphNode.Get()))
		{
			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.PinIcon"));
		}
		else if (Cast<UBehaviorTreeGraphNode_Decorator>(GraphNode.Get()))
		{
			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.RefPinIcon"));
		}
		else
		{
			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.FIB_Event"));
		}
	}

	return SNew(SImage)
		.Image(Brush)
		.ColorAndOpacity(IconColor);
}

FReply FFindInBTResult::OnClick(TWeakPtr<FBehaviorTreeEditor> BehaviorTreeEditor, TSharedPtr<FFindInBTResult> Root)
{
	const TSharedPtr<FBehaviorTreeEditor> BTEditorAsShared = BehaviorTreeEditor.Pin();
	FBehaviorTreeEditor* BTEditorPtr = BTEditorAsShared.Get();

	const TSharedPtr<FFindInBTResult> ParentAsShared = Parent.Pin();
	const FFindInBTResult* ParentPtr = ParentAsShared.Get();

	if (BTEditorPtr != nullptr && ParentPtr != nullptr)
	{
		if (ParentAsShared == Root)
		{
			BTEditorPtr->JumpToNode(GraphNode.Get());
		}
		else
		{
			BTEditorPtr->JumpToNode(ParentPtr->GraphNode.Get());
		}
	}

	return FReply::Handled();
}

FString FFindInBTResult::GetNodeTypeText() const
{
	if (GraphNode.IsValid())
	{
		FString NodeClassName = GraphNode->GetClass()->GetName();
		int32 Pos = NodeClassName.Find("_");
		if (Pos == INDEX_NONE)
		{
			return NodeClassName;
		}
		else
		{
			return NodeClassName.RightChop(Pos + 1);
		}
	}

	return FString();
}

FString FFindInBTResult::GetCommentText() const
{
	if (GraphNode.IsValid())
	{
		return GraphNode->NodeComment;
	}

	return FString();
}

//////////////////////////////////////////////////////////////////////////
// SFindInBT
void SFindInBT::Construct(const FArguments& InArgs, TSharedPtr<FBehaviorTreeEditor> InBehaviorTreeEditor)
{
	BehaviorTreeEditorPtr = InBehaviorTreeEditor;

	// initialize slate with new combo box for different search types
	SearchTypeComboBoxItems.Emplace(MakeShared<FString>(LOCTEXT("BTSearchType_Node", "Node").ToString()));
	SearchTypeComboBoxItems.Emplace(MakeShared<FString>(LOCTEXT("BTSearchType_BlackboardKey", "Blackboard Key").ToString()));
	SearchTypeComboBoxItems.Emplace(MakeShared<FString>(LOCTEXT("BTSearchType_GameplayTag", "GameplayTag").ToString()));

	TSharedPtr<FString> CurrentlySelected = SearchTypeComboBoxItems[0];

this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(0.25f)
				[
					SNew(STextComboBox)
					.OptionsSource(&SearchTypeComboBoxItems)
					.InitiallySelectedItem(CurrentlySelected)
					.OnSelectionChanged(this, &SFindInBT::OnSearchTypeSelectedItemChanged)
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				]
				+SHorizontalBox::Slot()
				.FillWidth(0.75f)
				[
					SAssignNew(SearchTextField, SSearchBox)
					.HintText(LOCTEXT("BehaviorTreeSearchHint", "Enter text to find nodes..."))
					.OnTextChanged(this, &SFindInBT::OnSearchTextChanged)
					.OnTextCommitted(this, &SFindInBT::OnSearchTextCommitted)
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.f, 4.f, 0.f, 0.f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				[
					SAssignNew(TreeView, STreeViewType)
					.TreeItemsSource(&ItemsFound)
					.OnGenerateRow(this, &SFindInBT::OnGenerateRow)
					.OnGetChildren(this, &SFindInBT::OnGetChildren)
					.OnSelectionChanged(this, &SFindInBT::OnTreeSelectionChanged)
					.SelectionMode(ESelectionMode::Multi)
				]
			]
		];
}

void SFindInBT::FocusForUse()
{
	// NOTE: Careful, GeneratePathToWidget can be reentrant in that it can call visibility delegates and such
	FWidgetPath FilterTextBoxWidgetPath;
	FSlateApplication::Get().GeneratePathToWidgetUnchecked(SearchTextField.ToSharedRef(), FilterTextBoxWidgetPath);

	// Set keyboard focus directly
	FSlateApplication::Get().SetKeyboardFocus(FilterTextBoxWidgetPath, EFocusCause::SetDirectly);
}

void SFindInBT::OnSearchTextChanged(const FText& Text)
{
	SearchValue = Text.ToString();

	InitiateSearch();
}

void SFindInBT::OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType)
{
	OnSearchTextChanged(Text);
}

void SFindInBT::InitiateSearch()
{
	TArray<FString> Tokens;
	SearchValue.ParseIntoArray(Tokens, TEXT(" "), true);

	for (auto It(ItemsFound.CreateIterator()); It; ++It)
	{
		(*It).Get()->SetNodeHighlight(false); // need to reset highlight
		TreeView->SetItemExpansion(*It, false);
	}
	ItemsFound.Empty();
	if (Tokens.Num() > 0)
	{
		HighlightText = FText::FromString(SearchValue);
		MatchTokens(Tokens);
	}

	// Insert a fake result to inform user if none found
	if (ItemsFound.Num() == 0)
	{
		ItemsFound.Add(FSearchResult(new FFindInBTResult(LOCTEXT("BehaviorTreeSearchNoResults", "No Results found").ToString())));
	}

	TreeView->RequestTreeRefresh();

	for (auto It(ItemsFound.CreateIterator()); It; ++It)
	{
		TreeView->SetItemExpansion(*It, true);
	}
}

void SFindInBT::MatchTokens(const TArray<FString>& Tokens)
{
	RootSearchResult.Reset();

	TWeakPtr<SGraphEditor> FocusedGraphEditor = BehaviorTreeEditorPtr.Pin()->GetFocusedGraphPtr();
	UEdGraph* Graph = NULL;
	if (FocusedGraphEditor.IsValid())
	{
		Graph = FocusedGraphEditor.Pin()->GetCurrentGraph();
	}

	if (Graph == NULL)
	{
		return;
	}

	RootSearchResult = FSearchResult(new FFindInBTResult(FString("BehaviorTreeRoot")));

	for (auto It(Graph->Nodes.CreateConstIterator()); It; ++It)
	{
		UEdGraphNode* Node = *It;

		const FString NodeName = Node->GetNodeTitle(ENodeTitleType::ListView).ToString();
		FSearchResult NodeResult(new FFindInBTResult(NodeName, RootSearchResult, Node));

		FString NodeSearchString = NodeName + Node->GetClass()->GetName() + Node->NodeComment;
		NodeSearchString = NodeSearchString.Replace(TEXT(" "), TEXT(""));

		bool bNodeMatchesSearch = StringMatchesSearchTokens(Tokens, NodeSearchString);

		UBehaviorTreeGraphNode* BTNode = Cast<UBehaviorTreeGraphNode>(Node);
		if (BTNode)
		{
			// search through node properties according to search type
			FString OutExactFieldValueFound;
			if (NodePropertyMatchesSearchTokens(Tokens, BTNode, OutExactFieldValueFound))
			{
				bNodeMatchesSearch = true;
				NodeResult->ExactFieldValueFound = OutExactFieldValueFound;
			}

			// searching through nodes' decorators
			for (auto DecoratorIt(BTNode->Decorators.CreateConstIterator()); DecoratorIt; ++DecoratorIt)
			{
				UBehaviorTreeGraphNode* Decorator = *DecoratorIt;
				MatchTokensInChild(Tokens, Decorator, NodeResult);
			}

			// searching through nodes' services
			for (auto ServiceIt(BTNode->Services.CreateConstIterator()); ServiceIt; ++ServiceIt)
			{
				UBehaviorTreeGraphNode* Service = *ServiceIt;
				MatchTokensInChild(Tokens, Service, NodeResult);
			}
		}

		if ((NodeResult->Children.Num() > 0) || bNodeMatchesSearch)
		{
			NodeResult->SetNodeHighlight(true);
			ItemsFound.Add(NodeResult);
		}
	}
}

void SFindInBT::MatchTokensInChild(const TArray<FString>& Tokens, UBehaviorTreeGraphNode* Child, FSearchResult ParentNode)
{
	if (Child == NULL)
	{
		return;
	}

	FString OutExactFieldValueFound;

	const FString ChildName = Child->GetNodeTitle(ENodeTitleType::ListView).ToString();
	FString ChildSearchString = ChildName + Child->GetClass()->GetName() + Child->NodeComment + GetNameSafe(Child->NodeInstance ? Child->NodeInstance->GetClass() : nullptr);
	ChildSearchString = ChildSearchString.Replace(TEXT(" "), TEXT(""));
	if (StringMatchesSearchTokens(Tokens, ChildSearchString) || NodePropertyMatchesSearchTokens(Tokens, Child, OutExactFieldValueFound))
	{
		FSearchResult DecoratorResult(new FFindInBTResult(ChildName, ParentNode, Child));
		DecoratorResult->ExactFieldValueFound = OutExactFieldValueFound; // save field found to include it when generating search results
		ParentNode->Children.Add(DecoratorResult);
	}
}

TSharedRef<ITableRow> SFindInBT::OnGenerateRow(FSearchResult InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow< TSharedPtr<FFindInBTResult> >, OwnerTable)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(450.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						InItem->CreateIcon()
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(2.f, 0.f)
					[
						SNew(STextBlock)
						.Text(FText::FromString(InItem->Value))
						.HighlightText(HighlightText)
					]
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(10.f, 0.f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(InItem->GetNodeTypeText()))
				.HighlightText(HighlightText)
			]
			// Include exact field found in search results
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(10.f, 0.f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(InItem->ExactFieldValueFound))
				.ColorAndOpacity(FLinearColor::Green)
				.HighlightText(HighlightText)
			] 
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(5.f, 0.f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(InItem->GetCommentText()))
				.ColorAndOpacity(FLinearColor::Yellow)
				.HighlightText(HighlightText)
			]
		];
}

void SFindInBT::OnGetChildren(FSearchResult InItem, TArray<FSearchResult>& OutChildren)
{
	OutChildren += InItem->Children;
}

void SFindInBT::OnTreeSelectionChanged(FSearchResult Item, ESelectInfo::Type)
{
	if (Item.IsValid())
	{
		Item->OnClick(BehaviorTreeEditorPtr, RootSearchResult);
	}
}

bool SFindInBT::StringMatchesSearchTokens(const TArray<FString>& Tokens, const FString& ComparisonString)
{
	bool bFoundAllTokens = true;

	// search the entry for each token, it must have all of them to pass
	for (auto TokIT(Tokens.CreateConstIterator()); TokIT; ++TokIT)
	{
		const FString& Token = *TokIT;
		if (!ComparisonString.Contains(Token))
		{
			bFoundAllTokens = false;
			break;
		}
	}
	return bFoundAllTokens;
}

bool SFindInBT::NodePropertyMatchesSearchTokens(const TArray<FString>& Tokens, UBehaviorTreeGraphNode* Node, FString& OutExactFieldValueFound) const
{
	if (Node->NodeInstance == nullptr)
	{
		return false;
	}

	// nothing to search for, so abort early
	if (SearchType == EFindInBTSearchType::Node)
	{
		return false;
	}

	for (FProperty* Property = Node->NodeInstance->GetClass()->PropertyLink;
		 Property != nullptr; Property = Property->PropertyLinkNext)
	{
		if (FieldPropertyMatchesSearchTokens(Tokens, Property, Node->NodeInstance, OutExactFieldValueFound))
		{
			return true;
		}
	}
	return false;
}

bool SFindInBT::FieldPropertyMatchesSearchTokens(const TArray<FString>& Tokens, const FProperty* Property, void* PropertySource, FString& OutExactFieldValueFound) const
{
	if (Property == nullptr)
	{
		return false;
	}

	if (PropertySource == nullptr)
	{
		return false;
	}

	const FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property);
	if (ArrayProp && ArrayProp->Inner)
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, ArrayProp->ContainerPtrToValuePtr<void>(PropertySource));
		for (int32 i = 0; i < ArrayHelper.Num(); ++i)
		{
			void* ArrayData = ArrayHelper.GetRawPtr(i);
			if (FieldPropertyMatchesSearchTokens(Tokens, ArrayProp->Inner, ArrayData, OutExactFieldValueFound))
			{
				return true;
			}
		}
	}

	const FStructProperty* StructProp = CastField<FStructProperty>(Property);
	if (StructProp && StructProp->Struct)
	{
		if (StructProp->Struct->IsChildOf(FBlackboardKeySelector::StaticStruct()))
		{
			if (SearchType != EFindInBTSearchType::BlackboardKey)
			{
				return false;
			}

			FBlackboardKeySelector* PropertyValue = Property->ContainerPtrToValuePtr<FBlackboardKeySelector>(PropertySource);
			if (PropertyValue == nullptr)
			{
				return false;
			}

			if (StringMatchesSearchTokens(Tokens, PropertyValue->SelectedKeyName.ToString()))
			{
				OutExactFieldValueFound = PropertyValue->SelectedKeyName.ToString();
				return true;
			}
		}
		else if (StructProp->Struct->IsChildOf(FGameplayTag::StaticStruct()))
		{
			if (SearchType != EFindInBTSearchType::GameplayTag)
			{
				return false;
			}

			FGameplayTag* PropertyValue = Property->ContainerPtrToValuePtr<FGameplayTag>(PropertySource);
			if (PropertyValue == nullptr)
			{
				return false;
			}

			if (StringMatchesSearchTokens(Tokens, PropertyValue->ToString()))
			{
				OutExactFieldValueFound = PropertyValue->ToString();
				return true;
			}
		}
		else
		{
			for (TFieldIterator<FProperty> iter(StructProp->Struct); iter; ++iter)
			{
				void* StructAddressInValuePtr = StructProp->ContainerPtrToValuePtr<void>(PropertySource);
				if (StructAddressInValuePtr == nullptr)
				{
					continue;
				}

				if (FieldPropertyMatchesSearchTokens(Tokens, *iter, StructAddressInValuePtr, OutExactFieldValueFound))
				{
					return true;
				}
			}
		}
	}
	else if (const FNameProperty* NameProperty = CastField<FNameProperty>(Property))
	{
		if (SearchType != EFindInBTSearchType::BlackboardKey)
		{
			return false;
		}

		FName* PropertyValue = NameProperty->ContainerPtrToValuePtr<FName>(PropertySource);
		if (PropertyValue == nullptr)
		{
			return false;
		}

		if (StringMatchesSearchTokens(Tokens, PropertyValue->ToString()))
		{
			OutExactFieldValueFound = PropertyValue->ToString();
			return true;
		}
	}

	return false;
}

void SFindInBT::OnSearchTypeSelectedItemChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	const int32 IndexOf = SearchTypeComboBoxItems.IndexOfByKey(NewValue);
	if (IndexOf != INDEX_NONE)
	{
		SearchType = static_cast<EFindInBTSearchType>(IndexOf);

		InitiateSearch();
	}
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
