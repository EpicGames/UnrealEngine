// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

class ITableRow;
class SWidget;
class UBehaviorTreeGraphNode;
class UEdGraphNode;

/** Indicates the kind of search we want to do */
enum class EFindInBTSearchType : uint8
{
	// Search for text on nodes
	Node,

	// Search through properties with matching Blackboard Key
	BlackboardKey,

	// Search through properties with matching GameplayTag
	GameplayTag
};

/** Item that matched the search results */
class FFindInBTResult
{
public: 
	/** Create a root (or only text) result */
	FFindInBTResult(const FString& InValue);
	
	/** Create a BT node result */
	FFindInBTResult(const FString& InValue, TSharedPtr<FFindInBTResult>& InParent, UEdGraphNode* InNode);

	/** Called when user clicks on the search item */
	FReply OnClick(TWeakPtr<class FBehaviorTreeEditor> BehaviorTreeEditor,  TSharedPtr<FFindInBTResult> Root);

	/** Create an icon to represent the result */
	TSharedRef<SWidget>	CreateIcon() const;

	/** Gets the comment on this node if any */
	FString GetCommentText() const;

	/** Gets the node type */
	FString GetNodeTypeText() const;

	/** Highlights BT tree nodes */
	void SetNodeHighlight(bool bHighlight);

	/** Any children listed under this BT node (decorators and services) */
	TArray< TSharedPtr<FFindInBTResult> > Children;

	/** The string value for this result */
	FString Value;

	/** The graph node that this search result refers to */
	TWeakObjectPtr<UEdGraphNode> GraphNode;

	/** Search result parent */
	TWeakPtr<FFindInBTResult> Parent;

	/** Show in search result row the exact field value found */
	FString ExactFieldValueFound;
};

/** Widget for searching for (BT nodes) across focused BehaviorTree */
class SFindInBT : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFindInBT){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<class FBehaviorTreeEditor> InBehaviorTreeEditor);

	/** Focuses this widget's search box */
	void FocusForUse();

private:
	typedef TSharedPtr<FFindInBTResult> FSearchResult;
	typedef STreeView<FSearchResult> STreeViewType;

	/** Called when user changes the text they are searching for */
	void OnSearchTextChanged(const FText& Text);

	/** Called when user commits text */
	void OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType);

	/** Get the children of a row */
	void OnGetChildren(FSearchResult InItem, TArray<FSearchResult>& OutChildren);

	/** Called when user clicks on a new result */
	void OnTreeSelectionChanged(FSearchResult Item, ESelectInfo::Type SelectInfo);

	/** Called when a new row is being generated */
	TSharedRef<ITableRow> OnGenerateRow(FSearchResult InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Begins the search based on the SearchValue */
	void InitiateSearch();
	
	/** Find any results that contain all of the tokens */
	void MatchTokens(const TArray<FString>& Tokens);

	/** Find if child contains all of the tokens and add a result accordingly */
	void MatchTokensInChild(const TArray<FString>& Tokens, UBehaviorTreeGraphNode* Child, FSearchResult ParentNode);
	
	/** Determines if a string matches the search tokens */
	static bool StringMatchesSearchTokens(const TArray<FString>& Tokens, const FString& ComparisonString);

	/** Determines if a string matches a node property according to the SearchType */
	bool NodePropertyMatchesSearchTokens(const TArray<FString>& Tokens, UBehaviorTreeGraphNode* Node, FString& OutExactFieldValueFound) const;

	/** Determines if a string matches a field property according to the SearchType */
	bool FieldPropertyMatchesSearchTokens(const TArray<FString>& Tokens, const FProperty* Property, void* Data, FString& OutExactFieldValueFound) const;

	/** Initiates search when SearchType changes */
	void OnSearchTypeSelectedItemChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);

private:
	/** Pointer back to the behavior tree editor that owns us */
	TWeakPtr<class FBehaviorTreeEditor> BehaviorTreeEditorPtr;
	
	/** The tree view displays the results */
	TSharedPtr<STreeViewType> TreeView;

	/** The search text box */
	TSharedPtr<class SSearchBox> SearchTextField;
	
	/** This buffer stores the currently displayed results */
	TArray<FSearchResult> ItemsFound;

	/** we need to keep a handle on the root result, because it won't show up in the tree */
	FSearchResult RootSearchResult;

	/** The string to highlight in the results */
	FText HighlightText;

	/** The string to search for */
	FString	SearchValue;

	/** Shared strings based on the search types */
	TArray<TSharedPtr<FString>> SearchTypeComboBoxItems;

	/** Current search type determines how search occurs */
	EFindInBTSearchType SearchType = EFindInBTSearchType::Node;
};
