// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBehaviorTreeBlackboardEditor.h"

#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "BehaviorTree/BlackboardData.h"
#include "BehaviorTreeEditorCommands.h"
#include "BehaviorTreeEditorDelegates.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphSchema.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "Modules/ModuleManager.h"
#include "SGraphActionMenu.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"

class SWidget;

#define LOCTEXT_NAMESPACE "SBehaviorTreeBlackboardEditor"

DEFINE_LOG_CATEGORY(LogBlackboardEditor);

void SBehaviorTreeBlackboardEditor::Construct(const FArguments& InArgs, TSharedRef<FUICommandList> InCommandList, UBlackboardData* InBlackboardData)
{
	OnEntrySelected = InArgs._OnEntrySelected;
	OnGetDebugKeyValue = InArgs._OnGetDebugKeyValue;
	OnIsDebuggerReady = InArgs._OnIsDebuggerReady;
	OnIsDebuggerPaused = InArgs._OnIsDebuggerPaused;
	OnGetDebugTimeStamp = InArgs._OnGetDebugTimeStamp;
	OnGetDisplayCurrentState = InArgs._OnGetDisplayCurrentState;
	OnIsBlackboardModeActive = InArgs._OnIsBlackboardModeActive;

	TSharedRef<FUICommandList> CommandList = MakeShareable(new FUICommandList);

	CommandList->MapAction(
		FBTBlackboardCommands::Get().DeleteEntry,
		FExecuteAction::CreateSP(this, &SBehaviorTreeBlackboardEditor::HandleDeleteEntry),
		FCanExecuteAction::CreateSP(this, &SBehaviorTreeBlackboardEditor::CanDeleteEntry)
		);

	CommandList->MapAction(
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SBehaviorTreeBlackboardEditor::HandleCopyEntry),
		FCanExecuteAction::CreateSP(this, &SBehaviorTreeBlackboardEditor::CanCopyEntry)
		);

	CommandList->MapAction(
		FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &SBehaviorTreeBlackboardEditor::HandlePasteEntry),
		FCanExecuteAction::CreateSP(this, &SBehaviorTreeBlackboardEditor::CanPasteEntry)
		);

	CommandList->MapAction(
		FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &SBehaviorTreeBlackboardEditor::HandleDuplicateEntry),
		FCanExecuteAction::CreateSP(this, &SBehaviorTreeBlackboardEditor::CanDuplicateEntry)
		);

	CommandList->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SBehaviorTreeBlackboardEditor::HandleRenameEntry),
		FCanExecuteAction::CreateSP(this, &SBehaviorTreeBlackboardEditor::CanRenameEntry)
		);

	InCommandList->Append(CommandList);

	SBehaviorTreeBlackboardView::Construct(
		SBehaviorTreeBlackboardView::FArguments()
		.OnEntrySelected(InArgs._OnEntrySelected)
		.OnGetDebugKeyValue(InArgs._OnGetDebugKeyValue)
		.OnGetDisplayCurrentState(InArgs._OnGetDisplayCurrentState)
		.OnIsDebuggerReady(InArgs._OnIsDebuggerReady)
		.OnIsDebuggerPaused(InArgs._OnIsDebuggerPaused)
		.OnGetDebugTimeStamp(InArgs._OnGetDebugTimeStamp)
		.IsReadOnly(false),
		CommandList,
		InBlackboardData
	);
}

void SBehaviorTreeBlackboardEditor::FillContextMenu(FMenuBuilder& MenuBuilder) const
{
	if (!IsDebuggerActive() && HasSelectedItems())
	{
		MenuBuilder.AddMenuEntry(FBTBlackboardCommands::Get().DeleteEntry);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy, NAME_None, LOCTEXT("Copy", "Copy"), LOCTEXT("Copy_Tooltip", "Copies this blackboard entry.") );
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste, NAME_None, LOCTEXT("Paste", "Paste"), LOCTEXT("Paste_Tooltip", "Pastes this blackboard entry.") );
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate, NAME_None, LOCTEXT("Duplicate", "Duplicate"), LOCTEXT("Duplicate_Tooltip", "Duplicates this blackboard entry.") );
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename, NAME_None, LOCTEXT("Rename", "Rename"), LOCTEXT("Rename_Tooltip", "Renames this blackboard entry.") );
	}
}

void SBehaviorTreeBlackboardEditor::FillToolbar(FToolBarBuilder& ToolbarBuilder) const
{
	ToolbarBuilder.AddComboButton(
		FUIAction(
			FExecuteAction(),
			FCanExecuteAction::CreateSP(this, &SBehaviorTreeBlackboardEditor::CanCreateNewEntry)
			), 
		FOnGetContent::CreateSP(this, &SBehaviorTreeBlackboardEditor::HandleCreateNewEntryMenu),
		LOCTEXT( "New_Label", "New Key" ),
		LOCTEXT( "New_ToolTip", "Create a new blackboard entry" ),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "BTEditor.Blackboard.NewEntry")
	);			
}

TSharedPtr<FExtender> SBehaviorTreeBlackboardEditor::GetToolbarExtender(TSharedRef<FUICommandList> ToolkitCommands) const
{
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	ToolbarExtender->AddToolBarExtension("Debugging", EExtensionHook::Before, ToolkitCommands, FToolBarExtensionDelegate::CreateSP( this, &SBehaviorTreeBlackboardEditor::FillToolbar ));

	return ToolbarExtender;
}

void SBehaviorTreeBlackboardEditor::HandleDeleteEntry()
{
	UBlackboardData* BlackboardDataPtr = BlackboardData.Get();
	if (!BlackboardDataPtr)
	{
		UE_LOGF(LogBlackboardEditor, Error, "Trying to delete an entry from a blackboard while no Blackboard Asset is set!");
		return;
	}

	if (!IsDebuggerActive())
	{
		bool bIsInherited = false;
		FBlackboardEntry* BlackboardEntry = GetSelectedEntry(bIsInherited);
		if (BlackboardEntry != nullptr && !bIsInherited)
		{
			const FScopedTransaction Transaction(LOCTEXT("BlackboardEntryDeleteTransaction", "Delete Blackboard Entry"));
			BlackboardDataPtr->SetFlags(RF_Transactional);
			BlackboardDataPtr->Modify();

			FProperty* KeysProperty = FindFProperty<FProperty>(UBlackboardData::StaticClass(), GET_MEMBER_NAME_CHECKED(UBlackboardData, Keys));
			BlackboardDataPtr->PreEditChange(KeysProperty);
		
			for (int32 ItemIndex = 0; ItemIndex < BlackboardDataPtr->Keys.Num(); ItemIndex++)
			{
				if (BlackboardEntry == &BlackboardDataPtr->Keys[ItemIndex])
				{
					BlackboardDataPtr->Keys.RemoveAt(ItemIndex);
					break;
				}
			}

			GraphActionMenu->RefreshAllActions(true);
			UE::BehaviorTreeEditor::Delegates::OnBlackboardKeyChanged.Broadcast(*BlackboardDataPtr, /* FBlackboardEntry* */ nullptr);

			// signal de-selection
			OnEntrySelected.ExecuteIfBound( /* FBlackboardEntry* */ nullptr, false);

			FPropertyChangedEvent PropertyChangedEvent(KeysProperty, EPropertyChangeType::ArrayRemove);
			BlackboardDataPtr->PostEditChangeProperty(PropertyChangedEvent);
		}
	}
}

namespace BehaviorTreeBlackboardEditorHelper
{
	constexpr const char* BlackboardEntryIdentifier = "BlackBoardEntry:";
	
	FName GetUniqueName(const UBlackboardData& BlackboardData, const FName& BaseName)
	{
		if (BlackboardData.GetKeyID(BaseName) != FBlackboard::InvalidKey)
		{
			FString BaseNameString = BaseName.ToString();
			FString UniqueName = BaseNameString + "Copy";
			int32 index = 1;
			while (BlackboardData.GetKeyID(FName(UniqueName)) != FBlackboard::InvalidKey)
			{
				UniqueName = BaseNameString + "Copy(" + FString::FromInt(index++) + ')';
			}
			return FName(UniqueName);
		}
		return BaseName;
	}
}

void SBehaviorTreeBlackboardEditor::HandleCopyEntry()
{
	if (IsDebuggerActive())
	{
		return;
	}

	UBlackboardData* BlackboardDataPtr = BlackboardData.Get();
	if (!BlackboardDataPtr)
	{
		UE_LOGF(LogBlackboardEditor, Error, "Trying to copy an entry from a blackboard while no Blackboard Asset is set!");
		return;
	}

	bool bIsInherited = false;
	FBlackboardEntry* BlackboardEntry = GetSelectedEntry(bIsInherited);
	if (BlackboardEntry != nullptr && !BlackboardEntry->EntryName.IsNone())
	{
		FString BlackboardEntryString;
		FBlackboardEntry DefaultValues;
		FBlackboardEntry::StaticStruct()->ExportText(BlackboardEntryString, BlackboardEntry, &DefaultValues, nullptr, PPF_SerializedAsImportText, nullptr);
		
		FPlatformApplicationMisc::ClipboardCopy(*(BehaviorTreeBlackboardEditorHelper::BlackboardEntryIdentifier + BlackboardEntryString));
	}
}

void SBehaviorTreeBlackboardEditor::HandlePasteEntry()
{
	UBlackboardData* BlackboardDataPtr = BlackboardData.Get();
	if (!BlackboardDataPtr)
	{
		UE_LOGF(LogBlackboardEditor, Error, "Trying to paste an entry from a blackboard while no Blackboard Asset is set!");
		return;
	}

	FString ClipboardMessage;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardMessage);
	if (!ClipboardMessage.RemoveFromStart(BehaviorTreeBlackboardEditorHelper::BlackboardEntryIdentifier))
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("BlackboardEntryPasteTransaction", "Paste Blackboard Entry"));
	BlackboardDataPtr->SetFlags(RF_Transactional);
	BlackboardDataPtr->Modify();
	
	FProperty* KeysProperty = FindFProperty<FProperty>(UBlackboardData::StaticClass(), GET_MEMBER_NAME_CHECKED(UBlackboardData, Keys));
	BlackboardDataPtr->PreEditChange(KeysProperty);
	
	FBlackboardEntry& NewEntry = BlackboardDataPtr->Keys.AddDefaulted_GetRef();

	if (FBlackboardEntry::StaticStruct()->ImportText(*ClipboardMessage, &NewEntry, BlackboardDataPtr, PPF_InstanceSubobjects, nullptr, FString()))
	{		
		NewEntry.EntryName = BehaviorTreeBlackboardEditorHelper::GetUniqueName(*BlackboardDataPtr, NewEntry.EntryName);
		GraphActionMenu->RefreshAllActions(true);
		UE::BehaviorTreeEditor::Delegates::OnBlackboardKeyChanged.Broadcast(*BlackboardDataPtr, &BlackboardDataPtr->Keys.Last());

		FPropertyChangedEvent PropertyChangedEvent(KeysProperty, EPropertyChangeType::ArrayAdd);
		BlackboardDataPtr->PostEditChangeProperty(PropertyChangedEvent);

		SetSelectedEntry(NewEntry.EntryName);
	}
	else
	{
		BlackboardDataPtr->Keys.Pop();
	}
}

void SBehaviorTreeBlackboardEditor::HandleDuplicateEntry()
{
	UBlackboardData* BlackboardDataPtr = BlackboardData.Get();
	if (!BlackboardDataPtr)
	{
		UE_LOGF(LogBlackboardEditor, Error, "Trying to duplicate an entry from a blackboard while no Blackboard Asset is set!");
		return;
	}

	if (!IsDebuggerActive())
	{
		bool bIsInherited = false;
		FBlackboardEntry* BlackboardEntry = GetSelectedEntry(bIsInherited);
		if (BlackboardEntry != nullptr)
		{
			const FScopedTransaction Transaction(LOCTEXT("BlackboardEntryDuplicateTransaction", "Duplicate Blackboard Entry"));
			BlackboardDataPtr->SetFlags(RF_Transactional);
			BlackboardDataPtr->Modify();

			FProperty* KeysProperty = FindFProperty<FProperty>(UBlackboardData::StaticClass(), GET_MEMBER_NAME_CHECKED(UBlackboardData, Keys));
			BlackboardDataPtr->PreEditChange(KeysProperty);

			FBlackboardEntry NewEntry = *BlackboardEntry;
			NewEntry.EntryName = BehaviorTreeBlackboardEditorHelper::GetUniqueName(*BlackboardDataPtr, BlackboardEntry->EntryName);
			NewEntry.KeyType = DuplicateObject<UBlackboardKeyType>(BlackboardEntry->KeyType, BlackboardDataPtr);
			BlackboardDataPtr->Keys.Add(NewEntry);

			GraphActionMenu->RefreshAllActions(true);
			UE::BehaviorTreeEditor::Delegates::OnBlackboardKeyChanged.Broadcast(*BlackboardDataPtr, &BlackboardDataPtr->Keys.Last());

			FPropertyChangedEvent PropertyChangedEvent(KeysProperty, EPropertyChangeType::ArrayAdd);
			BlackboardDataPtr->PostEditChangeProperty(PropertyChangedEvent);
			
			if (SetSelectedEntry(NewEntry.EntryName))
			{
				GraphActionMenu->OnRequestRenameOnActionNode();
			}
		}
	}
}

void SBehaviorTreeBlackboardEditor::HandleRenameEntry()
{
	if (!IsDebuggerActive())
	{
		GraphActionMenu->OnRequestRenameOnActionNode();
	}
}

bool SBehaviorTreeBlackboardEditor::CanDeleteEntry() const
{
	const bool bModeActive = OnIsBlackboardModeActive.IsBound() && OnIsBlackboardModeActive.Execute();

	if (!IsDebuggerActive() && bModeActive)
	{
		bool bIsInherited = false;
		FBlackboardEntry* BlackboardEntry = GetSelectedEntry(bIsInherited);
		if (BlackboardEntry != nullptr)
		{
			return !bIsInherited;
		}
	}

	return false;
}

bool SBehaviorTreeBlackboardEditor::CanCopyEntry() const
{
	const bool bModeActive = OnIsBlackboardModeActive.IsBound() && OnIsBlackboardModeActive.Execute();

	if (!IsDebuggerActive() && bModeActive)
	{
		bool bIsInherited = false;
		FBlackboardEntry* BlackboardEntry = GetSelectedEntry(bIsInherited);
		if (BlackboardEntry != nullptr)
		{
			return true;
		}
	}

	return false;
}

bool SBehaviorTreeBlackboardEditor::CanPasteEntry() const
{
	const bool bModeActive = OnIsBlackboardModeActive.IsBound() && OnIsBlackboardModeActive.Execute();

	if (!IsDebuggerActive() && bModeActive)
	{
		FString ClipboardMessage;
		FPlatformApplicationMisc::ClipboardPaste(ClipboardMessage);
		return ClipboardMessage.StartsWith(BehaviorTreeBlackboardEditorHelper::BlackboardEntryIdentifier);
	}

	return false;
}

bool SBehaviorTreeBlackboardEditor::CanDuplicateEntry() const
{
	const bool bModeActive = OnIsBlackboardModeActive.IsBound() && OnIsBlackboardModeActive.Execute();

	if (!IsDebuggerActive() && bModeActive)
	{
		bool bIsInherited = false;
		FBlackboardEntry* BlackboardEntry = GetSelectedEntry(bIsInherited);
		if (BlackboardEntry != nullptr)
		{
			return true;
		}
	}

	return false;
}

bool SBehaviorTreeBlackboardEditor::CanRenameEntry() const
{
	const bool bModeActive = OnIsBlackboardModeActive.IsBound() && OnIsBlackboardModeActive.Execute();

	if (!IsDebuggerActive() && bModeActive)
	{
		bool bIsInherited = false;
		FBlackboardEntry* BlackboardEntry = GetSelectedEntry(bIsInherited);
		if (BlackboardEntry != nullptr)
		{
			return !bIsInherited;
		}
	}

	return false;
}

class FBlackboardEntryClassFilter : public IClassViewerFilter
{
public:

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		if (InClass != nullptr)
		{
			return !InClass->HasAnyClassFlags(CLASS_Abstract | CLASS_HideDropDown) &&
				InClass->HasAnyClassFlags(CLASS_EditInlineNew) &&
				InClass->IsChildOf(UBlackboardKeyType::StaticClass());
		}
		return false;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return InUnloadedClassData->IsChildOf(UBlackboardKeyType::StaticClass());
	}
};


TSharedRef<SWidget> SBehaviorTreeBlackboardEditor::HandleCreateNewEntryMenu() const
{
	FClassViewerInitializationOptions Options;
	Options.bShowUnloadedBlueprints = true;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	Options.ClassFilters.Add(MakeShareable( new FBlackboardEntryClassFilter ));

	FOnClassPicked OnPicked( FOnClassPicked::CreateRaw( const_cast<SBehaviorTreeBlackboardEditor*>(this), &SBehaviorTreeBlackboardEditor::HandleKeyClassPicked ) );

	// clear the search box, just in case there's something typed in there 
	// We need to do that since key adding code takes advantage of selection mechanics
	TSharedRef<SEditableTextBox> FilterTextBox = GraphActionMenu->GetFilterTextBox();
	FilterTextBox->SetText(FText());

	return 
		SNew(SBox)
		.HeightOverride(240.0f)
		.WidthOverride(200.0f)
		[
			FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, OnPicked)
		];
}

void SBehaviorTreeBlackboardEditor::HandleKeyClassPicked(UClass* InClass)
{
	UBlackboardData* BlackboardDataPtr = BlackboardData.Get();
	if (BlackboardDataPtr == nullptr)
	{
		UE_LOGF(LogBlackboardEditor, Error, "Trying to delete an entry from a blackboard while no Blackboard Asset is set!");
		return;
	}

	FSlateApplication::Get().DismissAllMenus();

	check(InClass);
	check(InClass->IsChildOf(UBlackboardKeyType::StaticClass()));

	const FScopedTransaction Transaction(LOCTEXT("BlackboardEntryAddTransaction", "Add Blackboard Entry"));
	BlackboardDataPtr->SetFlags(RF_Transactional);
	BlackboardDataPtr->Modify();

	FProperty* KeysProperty = FindFProperty<FProperty>(UBlackboardData::StaticClass(), GET_MEMBER_NAME_CHECKED(UBlackboardData, Keys));
	BlackboardDataPtr->PreEditChange(KeysProperty);

	// create a name for this new key
	FString NewKeyName = InClass->GetDisplayNameText().ToString();
	NewKeyName = NewKeyName.Replace(TEXT(" "), TEXT(""));
	NewKeyName += TEXT("Key");

	int32 IndexSuffix = -1;
	auto DuplicateFunction = [&](const FBlackboardEntry& Key)
	{		
		if (Key.EntryName.ToString() == NewKeyName)
		{
			IndexSuffix = FMath::Max(0, IndexSuffix);
		}
		if (Key.EntryName.ToString().StartsWith(NewKeyName))
		{
			const FString ExistingSuffix = Key.EntryName.ToString().RightChop(NewKeyName.Len());
			if (ExistingSuffix.IsNumeric())
			{
				IndexSuffix = FMath::Max(FCString::Atoi(*ExistingSuffix) + 1, IndexSuffix);
			}
		}
	};

	// check for existing keys of the same name
	for (const auto& Key : BlackboardDataPtr->Keys) { DuplicateFunction(Key); };
	for (const auto& Key : BlackboardDataPtr->ParentKeys) { DuplicateFunction(Key); };

	if (IndexSuffix != -1)
	{
		NewKeyName += FString::Printf(TEXT("%d"), IndexSuffix);
	}

	FBlackboardEntry Entry;
	Entry.EntryName = FName(*NewKeyName);
	Entry.KeyType = NewObject<UBlackboardKeyType>(BlackboardDataPtr, InClass);

	BlackboardDataPtr->Keys.Add(Entry);

	GraphActionMenu->RefreshAllActions(true);
	UE::BehaviorTreeEditor::Delegates::OnBlackboardKeyChanged.Broadcast(*BlackboardDataPtr, &BlackboardDataPtr->Keys.Last());

	GraphActionMenu->SelectItemByName(Entry.EntryName, ESelectInfo::OnMouseClick);

	// Mark newly created entry as 'new'
	TArray< TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);
	check(SelectedActions.Num() == 1);
	check(SelectedActions[0]->GetTypeId() == FEdGraphSchemaAction_BlackboardEntry::StaticGetTypeId());
	TSharedPtr<FEdGraphSchemaAction_BlackboardEntry> BlackboardEntryAction = StaticCastSharedPtr<FEdGraphSchemaAction_BlackboardEntry>(SelectedActions[0]);
	BlackboardEntryAction->bIsNew = true;

	GraphActionMenu->OnRequestRenameOnActionNode();

	FPropertyChangedEvent PropertyChangedEvent(KeysProperty, EPropertyChangeType::ArrayAdd);
	BlackboardDataPtr->PostEditChangeProperty(PropertyChangedEvent);
}

bool SBehaviorTreeBlackboardEditor::CanCreateNewEntry() const
{
	if (OnIsDebuggerReady.IsBound())
	{
		return !OnIsDebuggerReady.Execute();
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
