// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Logging/LogMacros.h"
#include "SBehaviorTreeBlackboardView.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SWidget;
class UClass;

DECLARE_LOG_CATEGORY_EXTERN(LogBlackboardEditor, Warning, All);

class FExtender;
class FMenuBuilder;
class FToolBarBuilder;
class FUICommandList;
class UBlackboardData;

/** Delegate used to determine whether the Blackboard mode is active */
DECLARE_DELEGATE_RetVal(bool, FOnIsBlackboardModeActive);

/** Displays and edits blackboard entries */
class SBehaviorTreeBlackboardEditor : public SBehaviorTreeBlackboardView
{
public:
	SLATE_BEGIN_ARGS( SBehaviorTreeBlackboardEditor ) {}

		SLATE_EVENT(FOnEntrySelected, OnEntrySelected)
		SLATE_EVENT(FOnGetDebugKeyValue, OnGetDebugKeyValue)
		SLATE_EVENT(FOnGetDisplayCurrentState, OnGetDisplayCurrentState)
		SLATE_EVENT(FOnIsDebuggerReady, OnIsDebuggerReady)
		SLATE_EVENT(FOnIsDebuggerPaused, OnIsDebuggerPaused)
		SLATE_EVENT(FOnGetDebugTimeStamp, OnGetDebugTimeStamp)
		SLATE_EVENT(FOnIsBlackboardModeActive, OnIsBlackboardModeActive)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FUICommandList> InCommandList, UBlackboardData* InBlackboardData);

private:
	/** Fill the context menu with edit options */
	virtual void FillContextMenu(FMenuBuilder& MenuBuilder) const override;

	/** Fill the toolbar with edit options */
	void FillToolbar(FToolBarBuilder& ToolbarBuilder) const;

	/** Extend the toolbar */
	virtual TSharedPtr<FExtender> GetToolbarExtender(TSharedRef<FUICommandList> ToolkitCommands) const override;

	/** Handle deleting the currently selected entry */
	void HandleDeleteEntry();

	/** Handle copying the currently selected entry */
	void HandleCopyEntry();

	/** Handle pasting the currently copied entry */
	void HandlePasteEntry();
	
	/** Handle duplicating the currently selected entry */
	void HandleDuplicateEntry();

	/** Handle renaming the currently selected entry */
	void HandleRenameEntry();

	/** Create the menu used to create a new blackboard entry */
	TSharedRef<SWidget> HandleCreateNewEntryMenu() const;

	/** Create a new blackboard entry when a class is picked */
	void HandleKeyClassPicked(UClass* InClass);

	/** Delegate handler that disallows creating a new entry while the debugger is running */
	bool CanCreateNewEntry() const;

	/** Check whether the 'Delete' operation can be performed on the selected item  */
	bool CanDeleteEntry() const;

	/** Check whether the 'Copy' operation can be performed on the selected item  */
	bool CanCopyEntry() const;

	/** Check whether the 'Paste' operation can be performed on the copied item  */
	bool CanPasteEntry() const;
	
	/** Check whether the 'Duplicate' operation can be performed on the selected item  */
	bool CanDuplicateEntry() const;

	/** Check whether the 'Rename' operation can be performed on the selected item  */
	bool CanRenameEntry() const;

private:
	/** Delegate used to determine whether the Blackboard mode is active */
	FOnIsBlackboardModeActive OnIsBlackboardModeActive;
};
