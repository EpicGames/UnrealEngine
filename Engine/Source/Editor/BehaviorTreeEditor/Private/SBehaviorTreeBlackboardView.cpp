// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBehaviorTreeBlackboardView.h"

#include "BehaviorTreeEditorDelegates.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "BehaviorTree/BlackboardAssetProvider.h"
#include "BehaviorTree/BlackboardData.h"
#include "BehaviorTreeEditorCommands.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/Attribute.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "SBehaviorTreeBlackboardEditor.h"
#include "SGraphActionMenu.h"
#include "SGraphPalette.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/Casts.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SBehaviorTreeBlackboardView"

namespace EBlackboardSectionTitles
{
	enum Type
	{
		InheritedKeys = 1,
		Keys,
	};
}

FName FEdGraphSchemaAction_BlackboardEntry::StaticGetTypeId() 
{ 
	static FName Type("FEdGraphSchemaAction_BlackboardEntry"); return Type; 
}

FName FEdGraphSchemaAction_BlackboardEntry::GetTypeId() const 
{ 
	return StaticGetTypeId(); 
}

FEdGraphSchemaAction_BlackboardEntry::FEdGraphSchemaAction_BlackboardEntry( TWeakObjectPtr<UBlackboardData> InBlackboardData, FBlackboardEntry& InKey, bool bInIsInherited )
	: FEdGraphSchemaAction_Dummy()
	, BlackboardData(InBlackboardData)
	, Key(InKey)
	, bIsInherited(bInIsInherited)
	, bIsNew(false)
{
	check(BlackboardData.IsValid());
	Update();
}

void FEdGraphSchemaAction_BlackboardEntry::Update()
{
	UpdateSearchData(FText::FromName(Key.EntryName), FText::Format(LOCTEXT("BlackboardEntryFormat", "{0} '{1}'"), Key.KeyType ? Key.KeyType->GetClass()->GetDisplayNameText() : LOCTEXT("NullKeyDesc", "None"), FText::FromName(Key.EntryName)), (Key.EntryCategory.IsNone() ? FText() : FText::FromName(Key.EntryCategory)), FText());
	SectionID = bIsInherited ? EBlackboardSectionTitles::InheritedKeys : EBlackboardSectionTitles::Keys;
}

class SBehaviorTreeBlackboardItem : public SGraphPaletteItem
{
	SLATE_BEGIN_ARGS( SBehaviorTreeBlackboardItem ) {}

		SLATE_EVENT(FOnGetDebugKeyValue, OnGetDebugKeyValue)
		SLATE_EVENT(FOnGetDisplayCurrentState, OnGetDisplayCurrentState)
		SLATE_EVENT(FOnIsDebuggerReady, OnIsDebuggerReady)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData)
	{
		OnGetDebugKeyValue = InArgs._OnGetDebugKeyValue;
		OnIsDebuggerReady = InArgs._OnIsDebuggerReady;
		OnGetDisplayCurrentState = InArgs._OnGetDisplayCurrentState;

		const FSlateFontInfo NameFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);

		check(InCreateData);
		check(InCreateData->Action.IsValid());

		TSharedPtr<FEdGraphSchemaAction> GraphAction = InCreateData->Action;
		check(GraphAction->GetTypeId() == FEdGraphSchemaAction_BlackboardEntry::StaticGetTypeId());
		TSharedPtr<FEdGraphSchemaAction_BlackboardEntry> BlackboardEntryAction = StaticCastSharedPtr<FEdGraphSchemaAction_BlackboardEntry>(GraphAction);

		ActionPtr = InCreateData->Action;
		
		FSlateBrush const* IconBrush   = FAppStyle::GetBrush(TEXT("NoBrush"));
		GetPaletteItemIcon(GraphAction, IconBrush);

		TSharedRef<SWidget> IconWidget = CreateIconWidget( GraphAction->GetTooltipDescription(), IconBrush, FLinearColor::White );
		TSharedRef<SWidget> NameSlotWidget = CreateTextSlotWidget(InCreateData, BlackboardEntryAction->bIsInherited );
		TSharedRef<SWidget> DebugSlotWidget = CreateDebugSlotWidget( NameFont );

		// Create the actual widget
		this->ChildSlot
		[
			SNew(SHorizontalBox)
			// Icon slot
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				IconWidget
			]
			// Name slot
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(3,0)
			[
				NameSlotWidget
			]
			// Debug info slot
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(3,0)
			[
				DebugSlotWidget
			]
		];
	}

private:
	void GetPaletteItemIcon(TSharedPtr<FEdGraphSchemaAction> InGraphAction, FSlateBrush const*& OutIconBrush)
	{
		check(InGraphAction.IsValid());
		check(InGraphAction->GetTypeId() == FEdGraphSchemaAction_BlackboardEntry::StaticGetTypeId());
		TSharedPtr<FEdGraphSchemaAction_BlackboardEntry> BlackboardEntryAction = StaticCastSharedPtr<FEdGraphSchemaAction_BlackboardEntry>(InGraphAction);

		if(BlackboardEntryAction->Key.KeyType)
		{
			OutIconBrush = FSlateIconFinder::FindIconBrushForClass(BlackboardEntryAction->Key.KeyType->GetClass());
		}
	}

	virtual TSharedRef<SWidget> CreateTextSlotWidget(FCreateWidgetForActionData* const InCreateData, TAttribute<bool> bInIsReadOnly ) override
	{
		check(InCreateData);

		TSharedPtr< SWidget > DisplayWidget;

		// Copy the mouse delegate binding if we want it
		if( InCreateData->bHandleMouseButtonDown )
		{
			MouseButtonDownDelegate = InCreateData->MouseButtonDownDelegate;
		}

		// If the creation data says read only, then it must be read only
		bIsReadOnly = InCreateData->bIsReadOnly || bInIsReadOnly.Get();

		InlineRenameWidget =
			SAssignNew(DisplayWidget, SInlineEditableTextBlock)
			.Text(this, &SBehaviorTreeBlackboardItem::GetDisplayText)
			.HighlightText(InCreateData->HighlightText)
			.ToolTipText(this, &SBehaviorTreeBlackboardItem::GetItemTooltip)
			.OnTextCommitted(this, &SBehaviorTreeBlackboardItem::OnNameTextCommitted)
			.OnVerifyTextChanged(this, &SBehaviorTreeBlackboardItem::OnNameTextVerifyChanged)
			.IsSelected( InCreateData->IsRowSelectedDelegate )
			.IsReadOnly(this, &SBehaviorTreeBlackboardItem::IsReadOnly);

		InCreateData->OnRenameRequest->BindSP( InlineRenameWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode );

		return DisplayWidget.ToSharedRef();
	}

	virtual FText GetItemTooltip() const override
	{
		return ActionPtr.Pin()->GetTooltipDescription();
	}

	virtual void OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit) override
	{
		check(ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_BlackboardEntry::StaticGetTypeId());
		
		const FString AsString = *NewText.ToString();

		if (AsString.Len() >= NAME_SIZE)
		{
			UE_LOGF(LogBlackboardEditor, Error, "%ls is not a valid Blackboard key name. Needs to be shorter than %d characters.", *NewText.ToString(), NAME_SIZE);
			return;
		}

		TSharedPtr<FEdGraphSchemaAction_BlackboardEntry> BlackboardEntryAction = StaticCastSharedPtr<FEdGraphSchemaAction_BlackboardEntry>(ActionPtr.Pin());

		FName OldName = BlackboardEntryAction->Key.EntryName;
		FName NewName = FName(*AsString);
		if(NewName != OldName)
		{
			TArray<UObject*> ExternalBTAssetsWithKeyReferences;
			UBlackboardData* BlackboardData = BlackboardEntryAction->BlackboardData.Get();
			if (BlackboardData)
			{
				if (!BlackboardEntryAction->bIsNew)
				{
					// Preload behavior trees before we transact otherwise they will add objects to 
					// the transaction buffer whether we change them or not.
					// Blueprint regeneration does this in UEdGraphNode::CreatePin.
					LoadReferencerBehaviorTrees(*(BlackboardData), ExternalBTAssetsWithKeyReferences);
				}

				const FScopedTransaction Transaction(LOCTEXT("BlackboardEntryRenameTransaction", "Rename Blackboard Entry"));
				BlackboardData->SetFlags(RF_Transactional);
				BlackboardData->Modify();
				BlackboardEntryAction->Key.EntryName = NewName;

				FProperty* KeysArrayProperty = FindFProperty<FProperty>(UBlackboardData::StaticClass(), GET_MEMBER_NAME_CHECKED(UBlackboardData, Keys));
				FProperty* NameProperty = FindFProperty<FProperty>(FBlackboardEntry::StaticStruct(), GET_MEMBER_NAME_CHECKED(FBlackboardEntry, EntryName));
				FEditPropertyChain PropertyChain;
				PropertyChain.AddHead(KeysArrayProperty);
				PropertyChain.AddTail(NameProperty);
				PropertyChain.SetActiveMemberPropertyNode(KeysArrayProperty);
				PropertyChain.SetActivePropertyNode(NameProperty);

				BlackboardData->PreEditChange(PropertyChain);

				BlackboardEntryAction->Update();

				UE::BehaviorTreeEditor::Delegates::OnBlackboardKeyChanged.Broadcast(*BlackboardData, &BlackboardEntryAction->Key);

				if (!BlackboardEntryAction->bIsNew)
				{
					UpdateExternalBlackboardKeyReferences(OldName, NewName, ExternalBTAssetsWithKeyReferences);
				}

				FPropertyChangedEvent PropertyChangedEvent(NameProperty, EPropertyChangeType::ValueSet);
				FPropertyChangedChainEvent PropertyChangedChainEvent(PropertyChain, PropertyChangedEvent);
				BlackboardData->PostEditChangeChainProperty(PropertyChangedChainEvent);
			}
		}

		BlackboardEntryAction->bIsNew = false;
	}

	void GetBlackboardOwnerClasses(TArray<const UClass*>& BlackboardOwnerClasses)
	{
		for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
		{
			class UClass* Class = *ClassIt;
			if (Class->ImplementsInterface(UBlackboardAssetProvider::StaticClass()))
			{
				BlackboardOwnerClasses.Add(Class);
			}
		}
	}

	void LoadReferencerBehaviorTrees(const UBlackboardData& InBlackboardData, TArray<UObject*>& OutExternalBTAssetsWithKeyReferences)
	{
		// Get classes and derived classes which implement UBlackboardAssetProvider.
		TArray<const UClass*> BlackboardOwnerClasses;
		GetBlackboardOwnerClasses(BlackboardOwnerClasses);

		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

		TArray<FName> ReferencerPackages;
		AssetRegistry.GetReferencers(InBlackboardData.GetOutermost()->GetFName(), ReferencerPackages, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);

		if (ReferencerPackages.Num())
		{
			FScopedSlowTask SlowTask((float)ReferencerPackages.Num(), LOCTEXT("UpdatingBehaviorTrees", "Updating behavior trees"));
			SlowTask.MakeDialog();

			for (const FName& ReferencerPackage : ReferencerPackages)
			{
				TArray<FAssetData> Assets;
				AssetRegistry.GetAssetsByPackageName(ReferencerPackage, Assets);

				for (const FAssetData& Asset : Assets)
				{
					if (BlackboardOwnerClasses.Find(Asset.GetClass()) != INDEX_NONE)
					{
						SlowTask.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("CheckingBehaviorTree", "Key renamed, loading {0}"), FText::FromName(Asset.AssetName)));

						UObject* AssetObject = Asset.GetAsset();
						const IBlackboardAssetProvider* BlackboardProvider = Cast<const IBlackboardAssetProvider>(AssetObject);
						if (BlackboardProvider && BlackboardProvider->GetBlackboardAsset() == &InBlackboardData)
						{
							OutExternalBTAssetsWithKeyReferences.Add(AssetObject);
						}
					}
				}
			}
		}
	}

	#define GET_STRUCT_NAME_CHECKED(StructName) \
		((void)sizeof(StructName), TEXT(#StructName))

	void UpdateExternalBlackboardKeyReferences(const FName& OldKey, const FName& NewKey, const TArray<UObject*>& InExternalBTAssetsWithKeyReferences) const
	{
		for (const UObject* Asset : InExternalBTAssetsWithKeyReferences)
		{
			// search all subobjects of this package for FBlackboardKeySelector structs and update as necessary
			TArray<UObject*> Objects;
			GetObjectsWithOuter(Asset->GetOutermost(), Objects);
			for (const auto& SubObject : Objects)
			{
				for (TFieldIterator<FStructProperty> It(SubObject->GetClass()); It; ++It)
				{
					if (It->GetCPPType(NULL, CPPF_None).Contains(GET_STRUCT_NAME_CHECKED(FBlackboardKeySelector)))
					{
						FBlackboardKeySelector* PropertyValue = (FBlackboardKeySelector*)(It->ContainerPtrToValuePtr<uint8>(SubObject));
						if (PropertyValue && PropertyValue->SelectedKeyName == OldKey)
						{
							SubObject->Modify();
							PropertyValue->SelectedKeyName = NewKey;
						}
					}
				}
			}
		}
	}

	virtual bool OnNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage) override
	{
		check(ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_BlackboardEntry::StaticGetTypeId());
		TSharedPtr<FEdGraphSchemaAction_BlackboardEntry> BlackboardEntryAction = StaticCastSharedPtr<FEdGraphSchemaAction_BlackboardEntry>(ActionPtr.Pin());

		const FString NewTextAsString = InNewText.ToString();

		// check for duplicate keys
		const UBlackboardData* BlackboardData = BlackboardEntryAction->BlackboardData.Get();
		if (BlackboardData)
		{
			for(const auto& Key : BlackboardData->Keys)
			{
				if(&BlackboardEntryAction->Key != &Key && Key.EntryName.ToString() == NewTextAsString)
				{
					OutErrorMessage = LOCTEXT("DuplicateKeyWarning", "A key of this name already exists.");
					return false;
				}
			}

			for(const auto& Key : BlackboardData->ParentKeys)
			{
				if(&BlackboardEntryAction->Key != &Key && Key.EntryName.ToString() == NewTextAsString)
				{
					OutErrorMessage = LOCTEXT("DuplicateParentKeyWarning", "An inherited key of this name already exists.");
					return false;
				}
			}
		}

		return true;
	}

	/** Create widget for displaying debug information about this blackboard entry */
	TSharedRef<SWidget> CreateDebugSlotWidget(const FSlateFontInfo& InFontInfo)
	{
		check(ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_BlackboardEntry::StaticGetTypeId());
		TSharedPtr<FEdGraphSchemaAction_BlackboardEntry> BlackboardEntryAction = StaticCastSharedPtr<FEdGraphSchemaAction_BlackboardEntry>(ActionPtr.Pin());

		return SNew(STextBlock)
			.Text(this, &SBehaviorTreeBlackboardItem::GetDebugTextValue, BlackboardEntryAction)
			.Visibility(this, &SBehaviorTreeBlackboardItem::GetDebugTextVisibility);
	}

	FText GetDebugTextValue(TSharedPtr<FEdGraphSchemaAction_BlackboardEntry> BlackboardEntry) const
	{
		check(BlackboardEntry.IsValid());
		if(OnGetDebugKeyValue.IsBound() && OnGetDisplayCurrentState.IsBound())
		{
			return OnGetDebugKeyValue.Execute(BlackboardEntry->Key.EntryName, OnGetDisplayCurrentState.Execute());
		}
		
		return FText();
	}

	EVisibility GetDebugTextVisibility() const
	{
		if(OnIsDebuggerReady.IsBound())
		{
			return OnIsDebuggerReady.Execute() ? EVisibility::Visible : EVisibility::Collapsed;
		}

		return EVisibility::Collapsed;
	}

	bool IsReadOnly() const
	{
		if(OnIsDebuggerReady.IsBound())
		{
			return bIsReadOnly || OnIsDebuggerReady.Execute();
		}

		return bIsReadOnly;
	}

private:
	/** Delegate used to retrieve debug data to display */
	FOnGetDebugKeyValue OnGetDebugKeyValue;

	/** Delegate used to determine whether the BT debugger is active */
	FOnIsDebuggerReady OnIsDebuggerReady;

	/** Delegate used to determine whether the BT debugger displaying the current state */
	FOnGetDisplayCurrentState OnGetDisplayCurrentState;

	/** Read-only flag */
	bool bIsReadOnly;
};


void SBehaviorTreeBlackboardView::Construct(const FArguments& InArgs, TSharedRef<FUICommandList> InCommandList, UBlackboardData* InBlackboardData)
{
	OnEntrySelected = InArgs._OnEntrySelected;
	OnGetDebugKeyValue = InArgs._OnGetDebugKeyValue;
	OnIsDebuggerReady = InArgs._OnIsDebuggerReady;
	OnIsDebuggerPaused = InArgs._OnIsDebuggerPaused;
	OnGetDebugTimeStamp = InArgs._OnGetDebugTimeStamp;
	OnGetDisplayCurrentState = InArgs._OnGetDisplayCurrentState;

	OnBlackboardKeyChangedDelegateHandle = UE::BehaviorTreeEditor::Delegates::OnBlackboardKeyChanged.AddSP(this, &SBehaviorTreeBlackboardView::HandleBlackboardKeyChanged);

	BlackboardData = InBlackboardData;

	bShowCurrentState = OnGetDisplayCurrentState.IsBound() ? OnGetDisplayCurrentState.Execute() : true;

	TSharedRef<FUICommandList> CommandList = MakeShareable(new FUICommandList);

	CommandList->MapAction(
		FBTDebuggerCommands::Get().CurrentValues,
		FUIAction(
			FExecuteAction::CreateSP(this, &SBehaviorTreeBlackboardView::HandleUseCurrentValues),
			FCanExecuteAction::CreateSP(this, &SBehaviorTreeBlackboardView::IsDebuggerPaused),
			FIsActionChecked::CreateSP(this, &SBehaviorTreeBlackboardView::IsUsingCurrentValues),
			FIsActionButtonVisible::CreateSP(this, &SBehaviorTreeBlackboardView::IsDebuggerActive)
			)
		);

	CommandList->MapAction(
		FBTDebuggerCommands::Get().SavedValues, 
		FUIAction(
			FExecuteAction::CreateSP(this, &SBehaviorTreeBlackboardView::HandleUseSavedValues),
			FCanExecuteAction::CreateSP(this, &SBehaviorTreeBlackboardView::IsDebuggerPaused),
			FIsActionChecked::CreateSP(this, &SBehaviorTreeBlackboardView::IsUsingSavedValues),
			FIsActionButtonVisible::CreateSP(this, &SBehaviorTreeBlackboardView::IsDebuggerActive)
			)
		);

	InCommandList->Append(CommandList);

	// build debug toolbar
	FToolBarBuilder ToolbarBuilder(CommandList, FMultiBoxCustomization::None, GetToolbarExtender(InCommandList));
	
	ToolbarBuilder.BeginSection(TEXT("Debugging"));
	{
		ToolbarBuilder.AddToolBarButton(FBTDebuggerCommands::Get().CurrentValues);
		ToolbarBuilder.AddToolBarButton(FBTDebuggerCommands::Get().SavedValues);
	}
	ToolbarBuilder.EndSection();

	ChildSlot
	[
		SNew(SBorder)
		.Padding(4.0f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 0.0f, 0.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					ToolbarBuilder.MakeWidget()
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SBehaviorTreeBlackboardView::GetDebugTimeStampText)
					.Visibility(this, &SBehaviorTreeBlackboardView::GetDebuggerToolbarVisibility)
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(GraphActionMenu, SGraphActionMenu, InArgs._IsReadOnly)
				.OnCreateWidgetForAction(this, &SBehaviorTreeBlackboardView::HandleCreateWidgetForAction)
				.OnCollectAllActions(this, &SBehaviorTreeBlackboardView::HandleCollectAllActions)
				.OnGetSectionTitle(this, &SBehaviorTreeBlackboardView::HandleGetSectionTitle)
				.OnActionSelected(this, &SBehaviorTreeBlackboardView::HandleActionSelected)
				.OnContextMenuOpening(this, &SBehaviorTreeBlackboardView::HandleContextMenuOpening, InCommandList)
				.OnActionMatchesName(this, &SBehaviorTreeBlackboardView::HandleActionMatchesName)
				.AlphaSortItems(GetDefault<UEditorPerProjectUserSettings>()->bDisplayBlackboardKeysInAlphabeticalOrder)
				.DefaultRowExpanderBaseIndentLevel(1)
				.AutoExpandActionMenu(true)
			]
		]
	];
}

SBehaviorTreeBlackboardView::~SBehaviorTreeBlackboardView()
{
	UE::BehaviorTreeEditor::Delegates::OnBlackboardKeyChanged.Remove(OnBlackboardKeyChangedDelegateHandle);
}

TSharedRef<SWidget> SBehaviorTreeBlackboardView::HandleCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	return SNew(SBehaviorTreeBlackboardItem, InCreateData)
		.OnIsDebuggerReady(OnIsDebuggerReady)
		.OnGetDebugKeyValue(OnGetDebugKeyValue)
		.OnGetDisplayCurrentState(this, &SBehaviorTreeBlackboardView::IsUsingCurrentValues);
}

void SBehaviorTreeBlackboardView::HandleCollectAllActions( FGraphActionListBuilderBase& GraphActionListBuilder )
{
	if(BlackboardData != nullptr)
	{
		for(auto& ParentKey : BlackboardData->ParentKeys)
		{
			GraphActionListBuilder.AddAction( MakeShareable(new FEdGraphSchemaAction_BlackboardEntry(BlackboardData, ParentKey, true)) );
		}

		for(auto& Key : BlackboardData->Keys)
		{
			GraphActionListBuilder.AddAction( MakeShareable(new FEdGraphSchemaAction_BlackboardEntry(BlackboardData, Key, false)) );
		}
	}
}

FText SBehaviorTreeBlackboardView::HandleGetSectionTitle(int32 SectionID) const
{
	switch(SectionID)
	{
	case EBlackboardSectionTitles::InheritedKeys:
		return LOCTEXT("InheritedKeysSectionLabel", "Inherited Keys");
	case EBlackboardSectionTitles::Keys:
		return LOCTEXT("KeysSectionLabel", "Keys");
	}

	return FText();
}

void SBehaviorTreeBlackboardView::HandleActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType) const
{
	if (InSelectionType == ESelectInfo::OnMouseClick  || InSelectionType == ESelectInfo::OnKeyPress || SelectedActions.Num() == 0)
	{
		if(SelectedActions.Num() > 0)
		{
			check(SelectedActions[0]->GetTypeId() == FEdGraphSchemaAction_BlackboardEntry::StaticGetTypeId());
			TSharedPtr<FEdGraphSchemaAction_BlackboardEntry> BlackboardEntry = StaticCastSharedPtr<FEdGraphSchemaAction_BlackboardEntry>(SelectedActions[0]);
			OnEntrySelected.ExecuteIfBound(&BlackboardEntry->Key, BlackboardEntry->bIsInherited);
		}
	}
}

TSharedPtr<FEdGraphSchemaAction_BlackboardEntry> SBehaviorTreeBlackboardView::GetSelectedEntryInternal() const
{
	TArray< TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);

	if(SelectedActions.Num() > 0)
	{
		check(SelectedActions[0]->GetTypeId() == FEdGraphSchemaAction_BlackboardEntry::StaticGetTypeId());
		return StaticCastSharedPtr<FEdGraphSchemaAction_BlackboardEntry>(SelectedActions[0]);
	}

	return TSharedPtr<FEdGraphSchemaAction_BlackboardEntry>();
}

int32 SBehaviorTreeBlackboardView::GetSelectedEntryIndex(bool& bOutIsInherited) const
{
	TSharedPtr<FEdGraphSchemaAction_BlackboardEntry> Entry = GetSelectedEntryInternal();
	if(Entry.IsValid())
	{
		bOutIsInherited = Entry->bIsInherited;
		FBlackboardEntry* BlackboardEntry = &Entry->Key;

		// check to see what entry index we are using
		const UBlackboardData* BlackboardDataPtr = BlackboardData.Get();
		if (BlackboardDataPtr)
		{
			const TArray<FBlackboardEntry>& EntryArray = bOutIsInherited ? BlackboardDataPtr->ParentKeys : BlackboardDataPtr->Keys;
			for (int32 Index = 0; Index < EntryArray.Num(); Index++)
			{
				if (BlackboardEntry == &EntryArray[Index])
				{
					return Index;
				}
			}
		}
	}

	return INDEX_NONE;
}

FBlackboardEntry* SBehaviorTreeBlackboardView::GetSelectedEntry(bool& bOutIsInherited) const
{
	TSharedPtr<FEdGraphSchemaAction_BlackboardEntry> Entry = GetSelectedEntryInternal();
	if(Entry.IsValid())
	{
		bOutIsInherited = Entry->bIsInherited;
		return &Entry->Key;
	}

	return nullptr;
}

bool SBehaviorTreeBlackboardView::SetSelectedEntry(const FName EntryName) const
{
	return GraphActionMenu->SelectItemByName(EntryName);
}

void SBehaviorTreeBlackboardView::SetObject(UBlackboardData* InBlackboardData)
{
	BlackboardData = InBlackboardData;
	GraphActionMenu->RefreshAllActions(true);
}

void SBehaviorTreeBlackboardView::HandleBlackboardKeyChanged(const UBlackboardData& InBlackboardData, FBlackboardEntry* const InKey)
{
	// OnBlackboardKeyChanged is a global delegate so refresh only if it's our asset.
	if (BlackboardData == &InBlackboardData)
	{
		GraphActionMenu->RefreshAllActions(true);
	}
}

TSharedPtr<SWidget> SBehaviorTreeBlackboardView::HandleContextMenuOpening(TSharedRef<FUICommandList> ToolkitCommands) const
{
	FMenuBuilder MenuBuilder(/* bInShouldCloseWindowAfterMenuSelection =*/true, ToolkitCommands);

	FillContextMenu(MenuBuilder);

	return MenuBuilder.MakeWidget();
}

void SBehaviorTreeBlackboardView::FillContextMenu(FMenuBuilder& MenuBuilder) const
{

}

TSharedPtr<FExtender> SBehaviorTreeBlackboardView::GetToolbarExtender(TSharedRef<FUICommandList> ToolkitCommands) const
{
	return TSharedPtr<FExtender>();
}

void SBehaviorTreeBlackboardView::HandleUseCurrentValues()
{
	bShowCurrentState = true;
}

void SBehaviorTreeBlackboardView::HandleUseSavedValues()
{
	bShowCurrentState = false;
}

FText SBehaviorTreeBlackboardView::GetDebugTimeStampText() const
{
	FText TimeStampText;

	if(OnGetDebugTimeStamp.IsBound())
	{
		TimeStampText = FText::Format(LOCTEXT("ToolbarTimeStamp", "Time Stamp: {0}"), FText::AsNumber(OnGetDebugTimeStamp.Execute(IsUsingCurrentValues())));
	}

	return TimeStampText;
}

EVisibility SBehaviorTreeBlackboardView::GetDebuggerToolbarVisibility() const
{
	if(OnIsDebuggerReady.IsBound())
	{
		return OnIsDebuggerReady.Execute() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}

bool SBehaviorTreeBlackboardView::IsUsingCurrentValues() const
{
	if(OnGetDisplayCurrentState.IsBound())
	{
		return OnGetDisplayCurrentState.Execute() || bShowCurrentState;
	}

	return bShowCurrentState;
}

bool SBehaviorTreeBlackboardView::IsUsingSavedValues() const
{
	return !IsUsingCurrentValues();
}

bool SBehaviorTreeBlackboardView::HasSelectedItems() const
{
	bool bIsInherited = false;
	return GetSelectedEntry(bIsInherited) != nullptr;
}

bool SBehaviorTreeBlackboardView::IsDebuggerActive() const
{
	if(OnIsDebuggerReady.IsBound())
	{
		return OnIsDebuggerReady.Execute();
	}

	return true;
}

bool SBehaviorTreeBlackboardView::IsDebuggerPaused() const
{
	if(OnIsDebuggerPaused.IsBound())
	{
		return OnIsDebuggerPaused.Execute();
	}

	return true;
}

bool SBehaviorTreeBlackboardView::HandleActionMatchesName(FEdGraphSchemaAction* InAction, const FName& InName) const
{
	check(InAction->GetTypeId() == FEdGraphSchemaAction_BlackboardEntry::StaticGetTypeId());
	FEdGraphSchemaAction_BlackboardEntry* BlackboardEntryAction = static_cast<FEdGraphSchemaAction_BlackboardEntry*>(InAction);
	return BlackboardEntryAction->Key.EntryName == InName;
}

#undef LOCTEXT_NAMESPACE
