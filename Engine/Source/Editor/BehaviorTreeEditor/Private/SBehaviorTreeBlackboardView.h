// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/GCObject.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FExtender;
class FMenuBuilder;
class FReferenceCollector;
class FUICommandList;
class SGraphActionMenu;
class SWidget;
class UBlackboardData;
struct FBlackboardEntry;
struct FCreateWidgetForActionData;

/** Delegate executed when an entry is selected */
DECLARE_DELEGATE_TwoParams( FOnEntrySelected, const FBlackboardEntry*, bool );

/** Delegate used to retrieve debug data to display */
DECLARE_DELEGATE_RetVal_TwoParams(FText, FOnGetDebugKeyValue, const FName& /* InKeyName */, bool /* bUseCurrentState */);

/** Delegate used to determine whether the BT debugger is active */
DECLARE_DELEGATE_RetVal(bool, FOnIsDebuggerReady);

/** Delegate used to determine whether the BT debugger is paused */
DECLARE_DELEGATE_RetVal(bool, FOnIsDebuggerPaused);

/** Delegate used to determine whether the BT debugger displaying the current state */
DECLARE_DELEGATE_RetVal(bool, FOnGetDisplayCurrentState);

/** Delegate used to get the debugger's current timestamp */
DECLARE_DELEGATE_RetVal_OneParam(double, FOnGetDebugTimeStamp, bool /* bUseCurrentState */);


/** Blackboard entry in the list */
class FEdGraphSchemaAction_BlackboardEntry : public FEdGraphSchemaAction_Dummy
{
public:
	static FName StaticGetTypeId();
	virtual FName GetTypeId() const;

	FEdGraphSchemaAction_BlackboardEntry( TWeakObjectPtr<UBlackboardData> InBlackboardData, FBlackboardEntry& InKey, bool bInIsInherited );

	void Update();

	/** Blackboard we reference our key in */
	TWeakObjectPtr<UBlackboardData> BlackboardData;

	/** Actual key */
	FBlackboardEntry& Key;

	/** Whether this entry came from a parent blackboard */
	bool bIsInherited;

	/** Temp flag for new items */
	bool bIsNew;

	virtual bool CanBeRenamed() const override
	{
		return true;
	}
};

/** Displays blackboard entries */
class SBehaviorTreeBlackboardView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SBehaviorTreeBlackboardView ) 
	{
		_IsReadOnly = true;
	}

		SLATE_EVENT(FOnEntrySelected, OnEntrySelected)
		SLATE_EVENT(FOnGetDebugKeyValue, OnGetDebugKeyValue)
		SLATE_EVENT(FOnGetDisplayCurrentState, OnGetDisplayCurrentState)
		SLATE_EVENT(FOnIsDebuggerReady, OnIsDebuggerReady)
		SLATE_EVENT(FOnIsDebuggerPaused, OnIsDebuggerPaused)
		SLATE_EVENT(FOnGetDebugTimeStamp, OnGetDebugTimeStamp)
		SLATE_ARGUMENT(bool, IsReadOnly)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FUICommandList> InCommandList, UBlackboardData* InBlackboardData);
	~SBehaviorTreeBlackboardView();

	/**
	 * Retrieves the blackboard item currently selected by the user.
	 * @param	bOutIsInherited		Whether the item's index is in the key or parent key array
	 * @return A pointer to the currently selected entry (nullptr if a category is selected, or nothing at all)
	 */
	FBlackboardEntry* GetSelectedEntry(bool& bOutIsInherited) const;
	
	/**
	 * Selects a blackboard item
	 * @param	EntryName	The name of blackboard item to select
	 * @return true if the item was selected
	 */
	bool SetSelectedEntry(FName EntryName) const;


	/**
	 * Retrieves the blackboard item index currently selected by the user.
	 * @param	bOutIsInherited		Whether the item's index is in the key or parent key array
	 * @return The selected index (or INDEX_NONE if nothing is selected)
	 */
	int32 GetSelectedEntryIndex(bool& bOutIsInherited) const;

	/** Set the object we are looking at */
	void SetObject(UBlackboardData* InBlackboardData);

protected:
	/** Delegate handler for when a blackboard key changes (added, removed, renamed) */
	void HandleBlackboardKeyChanged(const UBlackboardData& InBlackboardData, FBlackboardEntry* const InKey);

	/** Delegate handler used to generate a widget for an 'action' (key) in the list */
	TSharedRef<SWidget> HandleCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData);

	/** Delegate handler used to collect all 'actions' (keys) for display */
	void HandleCollectAllActions( FGraphActionListBuilderBase& GraphActionListBuilder );

	/** Get the title of the specified section ID */
	FText HandleGetSectionTitle(int32 SectionID) const;

	/** Delegate handler used when an action is selected */
	void HandleActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType) const;

	/** Delegate handler used to generate an action's context menu */
	TSharedPtr<SWidget> HandleContextMenuOpening(TSharedRef<FUICommandList> ToolkitCommands) const;

	/** Add any context menu items */
	virtual void FillContextMenu(FMenuBuilder& MenuBuilder) const;

	/** Extend the toolbar */
	virtual TSharedPtr<FExtender> GetToolbarExtender(TSharedRef<FUICommandList> ToolkitCommands) const;

	/** Delegate handler invoked when we want to use current values */
	void HandleUseCurrentValues();

	/** Delegate handler invoked when we want to use saved values */
	void HandleUseSavedValues();

	/** Delegate handler used by toolbar to get display text for timestamp */
	FText GetDebugTimeStampText() const;

	/** Get the visibility of the toolbar sections used when debugging */
	EVisibility GetDebuggerToolbarVisibility() const;

	/** Delegate handler used by toolbar to determine whether to display saved values */
	bool IsUsingCurrentValues() const;

	/** Delegate handler used by toolbar to determine whether to display current values */
	bool IsUsingSavedValues() const;

	/** Check whether we have any items selected */
	bool HasSelectedItems() const;

	/** Helper function for GetSelectedEntry() */
	TSharedPtr<FEdGraphSchemaAction_BlackboardEntry> GetSelectedEntryInternal() const;
	
	/** Delegate handler that shows various controls when the debugger is active */
	bool IsDebuggerActive() const;

	/** Delegate handler that enables various controls when the debugger is active and paused */
	bool IsDebuggerPaused() const;

	/** Delegate handler used to match an FName to an action in the list, used for renaming keys */
	bool HandleActionMatchesName(FEdGraphSchemaAction* InAction, const FName& InName) const;

protected:
	/** The blackboard we are editing/viewing */
	TWeakObjectPtr<UBlackboardData> BlackboardData;

	/** The list of blackboard entries */
	TSharedPtr<SGraphActionMenu> GraphActionMenu;

	/** Delegate executed when an entry is selected */
	FOnEntrySelected OnEntrySelected;

	/** Delegate used to retrieve debug data to display */
	FOnGetDebugKeyValue OnGetDebugKeyValue;

	/** Delegate used to determine whether the BT debugger displaying the current state */
	FOnGetDisplayCurrentState OnGetDisplayCurrentState;

	/** Delegate used to determine whether the BT debugger is active */
	FOnIsDebuggerReady OnIsDebuggerReady;

	/** Delegate used to determine whether the BT debugger is paused */
	FOnIsDebuggerPaused OnIsDebuggerPaused;

	/** Delegate used to get the debugger's current timestamp */
	FOnGetDebugTimeStamp OnGetDebugTimeStamp;

	/** Handle to the registered OnBlackboardKeyChanged delegate */
	FDelegateHandle OnBlackboardKeyChangedDelegateHandle;

	/** Whether we want to show the current or saved state */
	bool bShowCurrentState;
};
