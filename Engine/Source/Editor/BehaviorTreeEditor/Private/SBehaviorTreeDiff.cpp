// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBehaviorTreeDiff.h"

#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTreeEditorUtils.h"
#include "BehaviorTreeGraphNode.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "DetailsViewArgs.h"
#include "DiffResults.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphUtilities.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/Views/ITypedTableView.h"
#include "GenericPlatform/GenericApplication.h"
#include "GraphDiffControl.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformCrt.h"
#include "IDetailsView.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/Sorting.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;
class SWidget;
class UObject;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SBehaviorTreeDiff"

//////////////////////////////////////////////////////////////////////////
// FTreeDiffResultItem

struct FTreeDiffResultItem : public TSharedFromThis<FTreeDiffResultItem>
{
	/**
	 * Constructor
	 * @param InResult A difference result 
	 */
	FTreeDiffResultItem(const FDiffSingleResult& InResult): Result(InResult){}

	/**
	 * GenerateWidget for the diff item
	 * @return The Widget
	 */
	TSharedRef<SWidget>	GenerateWidget() const
	{
		FText ToolTip = Result.ToolTip;
		FSlateColor Color = Result.GetDisplayColor();
		FText Text = Result.DisplayString;
		if(Text.IsEmpty())
		{
			Text = LOCTEXT("DIF_UnknownDiff", "Unknown Diff");
			ToolTip = LOCTEXT("DIF_Confused", "There is an unspecified difference");
		}
		return SNew(STextBlock)
			.ToolTipText(ToolTip)
			.ColorAndOpacity(Color)
			.Text(Text);
	}

	// A result of a diff
	const FDiffSingleResult Result;
};


//////////////////////////////////////////////////////////////////////////
// FDiffListCommands

class FDiffListCommands : public TCommands<FDiffListCommands>
{
public:
	/** Constructor */
	FDiffListCommands() 
		: TCommands<FDiffListCommands>("DiffList", LOCTEXT("Diff", "Behavior Tree Diff"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	/** Initialize commands */
	virtual void RegisterCommands() override
	{
		UI_COMMAND(Previous, "Prev", "Go to previous difference", EUserInterfaceActionType::Button, FInputChord(EKeys::F7, EModifierKey::Control));
		UI_COMMAND(Next, "Next", "Go to next difference", EUserInterfaceActionType::Button, FInputChord(EKeys::F7));
	}

	/** Go to previous difference */
	TSharedPtr<FUICommandInfo> Previous;

	/** Go to next difference */
	TSharedPtr<FUICommandInfo> Next;
};

//////////////////////////////////////////////////////////////////////////
// SBehaviorTreeDiff

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SBehaviorTreeDiff::Construct( const FArguments& InArgs )
{
	LastPinTarget = NULL;
	LastOtherPinTarget = NULL;
	FoundDiffs = MakeShared<TArray<FDiffSingleResult>>();

	FDiffListCommands::Register();

	PanelOld.BehaviorTree = InArgs._BehaviorTreeOld;
	PanelNew.BehaviorTree = InArgs._BehaviorTreeNew;

	PanelOld.RevisionInfo = InArgs._OldRevision;
	PanelNew.RevisionInfo = InArgs._NewRevision;

	PanelOld.bShowAssetName = InArgs._ShowAssetNames;
	PanelNew.bShowAssetName = InArgs._ShowAssetNames;

	OpenInDefaults = InArgs._OpenInDefaults;

	TSharedRef<SHorizontalBox> DefaultEmptyPanel = SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BehaviorTreeDifGraphsToolTip", "Select Graph to Diff"))
		];

	this->ChildSlot
	[	
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Content()
		[
			SNew(SSplitter)
			+SSplitter::Slot()
			.Value(0.2f)
			[
				SNew(SBorder)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						//open in p4dif tool
						SNew(SButton)
						.OnClicked(this, &SBehaviorTreeDiff::OnOpenInDefaults)
						.Content()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("DifBehaviorTreeDefaults", "Default Diff"))
						]
					]
					+SVerticalBox::Slot()
					.FillHeight(1.f)
					[
						GenerateDiffListWidget()
					]
				]
			]
			+SSplitter::Slot()
			.Value(0.8f)
			[
				// Diff Window
				SNew(SSplitter)
				+SSplitter::Slot()
				.Value(0.5f)
				[
					// Left Diff
					SAssignNew(PanelOld.GraphEditorBorder, SBorder)
					.VAlign(VAlign_Fill)
					[
						DefaultEmptyPanel
					]
				]
				+SSplitter::Slot()
				.Value(0.5f)
				[
					// Right Diff
					SAssignNew(PanelNew.GraphEditorBorder, SBorder)
					.VAlign(VAlign_Fill)
					[
						DefaultEmptyPanel
					]
				]
			]
		]
	];

	PanelOld.GeneratePanel(PanelOld.BehaviorTree ? PanelOld.BehaviorTree->BTGraph : nullptr, FoundDiffs);
	PanelNew.GeneratePanel(PanelNew.BehaviorTree ? PanelNew.BehaviorTree->BTGraph : nullptr, FoundDiffs);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FReply SBehaviorTreeDiff::OnOpenInDefaults()
{
	OpenInDefaults.ExecuteIfBound(PanelOld.BehaviorTree, PanelNew.BehaviorTree);
	return FReply::Handled();
}

TSharedRef<SWidget> SBehaviorTreeDiff::GenerateDiffListWidget()
{
	BuildDiffSourceArray();
	if(DiffListSource.Num() > 0)
	{
		Algo::SortBy(DiffListSource, [](const FSharedDiffOnGraph& Data) { return Data->Result.Diff; });

		// Map commands through UI
		const FDiffListCommands& Commands = FDiffListCommands::Get();
		KeyCommands = MakeShareable(new FUICommandList );

		KeyCommands->MapAction(Commands.Previous, FExecuteAction::CreateSP(this, &SBehaviorTreeDiff::PrevDiff));
		KeyCommands->MapAction(Commands.Next, FExecuteAction::CreateSP(this, &SBehaviorTreeDiff::NextDiff));

		FToolBarBuilder ToolbarBuilder(KeyCommands.ToSharedRef(), FMultiBoxCustomization::None);
		ToolbarBuilder.AddToolBarButton(Commands.Previous, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDif.PrevDiff"));
		ToolbarBuilder.AddToolBarButton(Commands.Next, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDif.NextDiff"));

		TSharedRef<SHorizontalBox> Result =	SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(0.f)
			.AutoHeight()
			[
				ToolbarBuilder.MakeWidget()
			]
			+SVerticalBox::Slot()
			.Padding(0.f)
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("PropertyWindow.CategoryBackground"))
				.Padding(FMargin(2.0f))
				.ForegroundColor(FAppStyle::GetColor("PropertyWindow.CategoryForeground"))
				.ToolTipText(LOCTEXT("BehvaiorTreeDifDifferencesToolTip", "List of differences found between revisions, click to select"))
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("RevisionDifferences", "Revision Differences"))
				]
			]
			+SVerticalBox::Slot()
			.Padding(1.f)
			.FillHeight(1.f)
			[
				SAssignNew(DiffList, SListViewType)
				.ListItemsSource(&DiffListSource)
				.OnGenerateRow(this, &SBehaviorTreeDiff::OnGenerateRow)
				.SelectionMode(ESelectionMode::Single)
				.OnSelectionChanged(this, &SBehaviorTreeDiff::OnSelectionChanged)
			]
		];
		return Result;
	}
	else
	{
		return SNew(SBorder).Visibility(EVisibility::Hidden);
	}
}

void SBehaviorTreeDiff::BuildDiffSourceArray()
{
	FoundDiffs->Empty();
	DiffListSource.Empty();
	if (!PanelOld.BehaviorTree || !PanelNew.BehaviorTree)
	{
		return;
	}
	
	FGraphDiffControl::DiffGraphs(PanelOld.BehaviorTree->BTGraph, PanelNew.BehaviorTree->BTGraph, *FoundDiffs);

	for (auto DiffIt(FoundDiffs->CreateConstIterator()); DiffIt; ++DiffIt)
	{
		DiffListSource.Add(FSharedDiffOnGraph(new FTreeDiffResultItem(*DiffIt)));
	}
}

void SBehaviorTreeDiff::NextDiff()
{
	int32 Index = (GetCurrentDiffIndex() + 1) % DiffListSource.Num();
	DiffList->SetSelection(DiffListSource[Index]);
}

void SBehaviorTreeDiff::PrevDiff()
{
	int32 Index = GetCurrentDiffIndex();
	if(Index == 0)
	{
		Index = DiffListSource.Num() - 1;
	}
	else
	{
		Index = (Index - 1) % DiffListSource.Num();
	}
	DiffList->SetSelection(DiffListSource[Index]);
}

int32 SBehaviorTreeDiff::GetCurrentDiffIndex() 
{
	auto Selected = DiffList->GetSelectedItems();
	if(Selected.Num() == 1)
	{	
		int32 Index = 0;
		for(auto It(DiffListSource.CreateIterator());It;++It,Index++)
		{
			if(*It == Selected[0])
			{
				return Index;
			}
		}
	}
	return 0;
}

TSharedRef<ITableRow> SBehaviorTreeDiff::OnGenerateRow(FSharedDiffOnGraph Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return	SNew(STableRow< FSharedDiffOnGraph >, OwnerTable)
			.Content()
			[
				Item->GenerateWidget()
			];
}

void SBehaviorTreeDiff::OnSelectionChanged(FSharedDiffOnGraph Item, ESelectInfo::Type SelectionType)
{
	if(!Item.IsValid())
	{
		return;
	}

	//focus the graph onto the diff that was clicked on
	FDiffSingleResult Result = Item->Result;
	if(Result.Pin1)
	{
		PanelNew.GraphEditor.Pin()->ClearSelectionSet();
		PanelOld.GraphEditor.Pin()->ClearSelectionSet();

		auto FocusPin = [this](UEdGraphPin* InPin)
		{
			if (InPin)
			{
				LastPinTarget = InPin;

				UEdGraph* NodeGraph = InPin->GetOwningNode()->GetGraph();
				SGraphEditor* NodeGraphEditor = GetGraphEditorForGraph(NodeGraph);
				NodeGraphEditor->JumpToPin(InPin);
			}
		};

		FocusPin(Result.Pin1);
		FocusPin(Result.Pin2);
	}
	else if(Result.Node1)
	{
		PanelNew.GraphEditor.Pin()->ClearSelectionSet();
		PanelOld.GraphEditor.Pin()->ClearSelectionSet();

		auto FocusNode = [this](UEdGraphNode* InNode)
		{
			if (InNode)
			{
				UEdGraph* NodeGraph = InNode->GetGraph();
				SGraphEditor* NodeGraphEditor = GetGraphEditorForGraph(NodeGraph);

				UBehaviorTreeGraphNode* BTNode = Cast<UBehaviorTreeGraphNode>(InNode);
				if (BTNode && BTNode->bIsSubNode)
				{
					// This is a sub-node, we need to find our parent node in the graph
					// todo: work out why BTNode->ParentNode is always null
					TObjectPtr<UEdGraphNode>* ParentNodePtr = NodeGraph->Nodes.FindByPredicate([BTNode](UEdGraphNode* PotentialParentNode) -> bool
					{
						UBehaviorTreeGraphNode* BTPotentialParentNode = Cast<UBehaviorTreeGraphNode>(PotentialParentNode);
						return BTPotentialParentNode && (BTPotentialParentNode->Decorators.Contains(BTNode) || BTPotentialParentNode->Services.Contains(BTNode));
					});

					// We need to call JumpToNode on the parent node, and then SetNodeSelection on the sub-node
					// as JumpToNode doesn't work for sub-nodes
					if (ParentNodePtr)
					{
						check(InNode->GetGraph() == (*ParentNodePtr)->GetGraph());
						NodeGraphEditor->JumpToNode(*ParentNodePtr, false, false);
					}
					NodeGraphEditor->SetNodeSelection(InNode, true);
				}
				else
				{
					NodeGraphEditor->JumpToNode(InNode, false);
				}
			}
		};

		FocusNode(Result.Node1);
		FocusNode(Result.Node2);
	}
}

SGraphEditor* SBehaviorTreeDiff::GetGraphEditorForGraph(UEdGraph* Graph) const
{
	if(PanelOld.GraphEditor.Pin()->GetCurrentGraph() == Graph)
	{
		return PanelOld.GraphEditor.Pin().Get();
	}
	else if(PanelNew.GraphEditor.Pin()->GetCurrentGraph() == Graph)
	{
		return PanelNew.GraphEditor.Pin().Get();
	}
	checkNoEntry();
	return NULL;
}


//////////////////////////////////////////////////////////////////////////
// FBehaviorTreeDiffPanel
//////////////////////////////////////////////////////////////////////////

SBehaviorTreeDiff::FBehaviorTreeDiffPanel::FBehaviorTreeDiffPanel()
{
	BehaviorTree = NULL;
}

void SBehaviorTreeDiff::FBehaviorTreeDiffPanel::GeneratePanel(UEdGraph* Graph, TSharedPtr<TArray<FDiffSingleResult>> DiffResults)
{
	TSharedPtr<SWidget> Widget = SNew(SBorder)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock).Text( LOCTEXT("BTDifPanelNoGraphTip", "Graph does not exist in this revision"))
		];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	DetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );
	DetailsView->SetObject( NULL );
	DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateRaw(this, &SBehaviorTreeDiff::FBehaviorTreeDiffPanel::IsPropertyEditable));
	
	if(Graph)
	{
		SGraphEditor::FGraphEditorEvents InEvents;
		InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateRaw(this, &SBehaviorTreeDiff::FBehaviorTreeDiffPanel::OnSelectionChanged);

		FGraphAppearanceInfo AppearanceInfo;
		AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_BehaviorDif", "DIFF");

		if (!GraphEditorCommands.IsValid())
		{
			GraphEditorCommands = MakeShareable(new FUICommandList());

			GraphEditorCommands->MapAction(FGenericCommands::Get().Copy,
				FExecuteAction::CreateRaw(this, &FBehaviorTreeDiffPanel::CopySelectedNodes),
				FCanExecuteAction::CreateRaw(this, &FBehaviorTreeDiffPanel::CanCopyNodes));
		}

		auto Editor = SNew(SGraphEditor)
			.AdditionalCommands(GraphEditorCommands)
			.GraphToEdit(Graph)
			.DiffResults(DiffResults)
			.IsEditable(false)
			.TitleBar(SNew(SBorder).HAlign(HAlign_Center)
			[
				SNew(STextBlock).Text(GetTitle())
			])
			.Appearance(AppearanceInfo)
			.GraphEvents(InEvents);

		const FSlateBrush* ContentAreaBrush = FAppStyle::GetBrush( "Docking.Tab", ".ContentAreaBrush" );

		auto NewWidget = SNew(SSplitter)
			.Orientation(Orient_Vertical)
			+SSplitter::Slot()
			.Value(0.8f)
			[
				Editor
			]
			+SSplitter::Slot()
			.Value(0.2f)
			[
				SNew( SBorder )
				.Visibility( EVisibility::Visible )
				.BorderImage( ContentAreaBrush )
				[
					DetailsView.ToSharedRef()
				]
			];

		GraphEditor = Editor;
		Widget = NewWidget;
	}

	GraphEditorBorder->SetContent(Widget.ToSharedRef());
}

FText SBehaviorTreeDiff::FBehaviorTreeDiffPanel::GetTitle() const
{
	FText Title = LOCTEXT("CurrentRevision", "Current Revision");

	// if this isn't the current working version being displayed
	if (!RevisionInfo.Revision.IsEmpty())
	{
		// Don't use grouping on the revision or CL numbers to match how Perforce displays them
		const FText DateText = FText::AsDate(RevisionInfo.Date, EDateTimeStyle::Short);
		const FText RevisionText = FText::FromString(RevisionInfo.Revision);
		const FText ChangelistText = FText::AsNumber(RevisionInfo.Changelist, &FNumberFormattingOptions::DefaultNoGrouping());

		if (bShowAssetName && BehaviorTree)
		{
			FString AssetName = BehaviorTree->GetName();
			if(ISourceControlModule::Get().GetProvider().UsesChangelists())
			{
				FText LocalizedFormat = LOCTEXT("NamedRevisionDiffFmtUsesChangelists", "{0} - Revision {1}, CL {2}, {3}");
				Title = FText::Format(LocalizedFormat, FText::FromString(AssetName), RevisionText, ChangelistText, DateText);
			}
			else
			{
				FText LocalizedFormat = LOCTEXT("NamedRevisionDiffFmt", "{0} - Revision {1}, {2}");
				Title = FText::Format(LocalizedFormat, FText::FromString(AssetName), RevisionText, DateText);
			}
		}
		else
		{
			if(ISourceControlModule::Get().GetProvider().UsesChangelists())
			{
				FText LocalizedFormat = LOCTEXT("PreviousRevisionDifFmtUsesChangelists", "Revision {0}, CL {1}, {2}");
				Title = FText::Format(LocalizedFormat, RevisionText, ChangelistText, DateText);
			}
			else
			{
				FText LocalizedFormat = LOCTEXT("PreviousRevisionDifFmt", "Revision {0}, {2}");
				Title = FText::Format(LocalizedFormat, RevisionText, DateText);
			}
		}
	}
	else if (bShowAssetName && BehaviorTree)
	{
		FString AssetName = BehaviorTree->GetName();
		FText LocalizedFormat = LOCTEXT("NamedCurrentRevisionFmt", "{0} - Current Revision");
		Title = FText::Format(LocalizedFormat, FText::FromString(AssetName));
	}

	return Title;
}

FGraphPanelSelectionSet SBehaviorTreeDiff::FBehaviorTreeDiffPanel::GetSelectedNodes() const
{
	FGraphPanelSelectionSet CurrentSelection;
	TSharedPtr<SGraphEditor> FocusedGraphEd = GraphEditor.Pin();
	if (FocusedGraphEd.IsValid())
	{
		CurrentSelection = FocusedGraphEd->GetSelectedNodes();
	}
	return CurrentSelection;
}

void SBehaviorTreeDiff::FBehaviorTreeDiffPanel::CopySelectedNodes()
{
	// Export the selected nodes and place the text on the clipboard
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	FString ExportedText;
	FEdGraphUtilities::ExportNodesToText(SelectedNodes, ExportedText);
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
}

bool SBehaviorTreeDiff::FBehaviorTreeDiffPanel::CanCopyNodes() const
{
	// If any of the nodes can be duplicated then we should allow copying
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if ((Node != NULL) && Node->CanDuplicateNode())
		{
			return true;
		}
	}
	return false;
}

void SBehaviorTreeDiff::FBehaviorTreeDiffPanel::OnSelectionChanged( const FGraphPanelSelectionSet& NewSelection )
{
	BehaviorTreeEditorUtils::FPropertySelectionInfo SelectionInfo;
	TArray<UObject*> Selection = BehaviorTreeEditorUtils::GetSelectionForPropertyEditor(NewSelection, SelectionInfo);

	if (Selection.Num() == 1)
	{
		if (DetailsView.IsValid())
		{
			DetailsView->SetObjects(Selection);
		}
	}
	else if (DetailsView.IsValid())
	{
		DetailsView->SetObject(NULL);
	}
}

bool SBehaviorTreeDiff::FBehaviorTreeDiffPanel::IsPropertyEditable()
{
	return false;
}

#undef LOCTEXT_NAMESPACE
