// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "GraphEditor.h"
#include "HAL/Platform.h"
#include "IAssetTypeActions.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;
class ITableRow;
class SBorder;
class STableViewBase;
class SWidget;
class UEdGraph;
class UEdGraphPin;
struct FDiffSingleResult;
template <typename ItemType> class SListView;

class SBehaviorTreeDiff: public SCompoundWidget
{
public:

	// Delegate for default Diff tool
	DECLARE_DELEGATE_TwoParams(FOpenInDefaults, class UBehaviorTree* , class UBehaviorTree*);

	SLATE_BEGIN_ARGS(SBehaviorTreeDiff ){}
		SLATE_ARGUMENT(class UBehaviorTree*, BehaviorTreeOld)
		SLATE_ARGUMENT(class UBehaviorTree*, BehaviorTreeNew)
		SLATE_ARGUMENT(struct FRevisionInfo, OldRevision)
		SLATE_ARGUMENT(struct FRevisionInfo, NewRevision)
		SLATE_ARGUMENT(bool, ShowAssetNames)
		SLATE_EVENT(FOpenInDefaults, OpenInDefaults)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	//Panel used to display the behavior tree
	struct FBehaviorTreeDiffPanel
	{
		/** Constructor */
		FBehaviorTreeDiffPanel();

		/** 
		 * Generate a panel that displays the Graph and reflects the items in the DiffResults
		 * @param Graph        Graph Being Displayed
		 * @param DiffResults  Attribute for obtaining the list of differences
		 */
		void GeneratePanel(UEdGraph* Graph, TSharedPtr<TArray<FDiffSingleResult>> DiffResults);

		/**
		 * Returns title for this panel
		 * @return The Title
		 */
		FText GetTitle() const;

		/** 
		 * Called when user hits keyboard shortcut to copy nodes
		 */
		void CopySelectedNodes();

		/**
		 * Called When graph node gains focus
		 */
		void OnSelectionChanged( const FGraphPanelSelectionSet& Selection );

		/**
		 * Delegate to say if a node property should be editable
		 */
		bool IsPropertyEditable();

		/**
		 * Gets whatever nodes are selected in the Graph Editor
		 * @return Gets the selected nodes in the graph
		 */
		FGraphPanelSelectionSet GetSelectedNodes() const;

		/**
		 * Can user copy any of the selected nodes?
		 * @return True if can copy
		 */
		bool CanCopyNodes() const;

		// The behavior Tree that owns the graph we are showing
		class UBehaviorTree* BehaviorTree;

		// Revision information for this behavior tree
		FRevisionInfo RevisionInfo;

		// The border around the graph editor, used to change the content when new graphs are set
		TSharedPtr<SBorder> GraphEditorBorder;

		// The graph editor which does the work of displaying the graph
		TWeakPtr<class SGraphEditor> GraphEditor;

		// If we should show a name identifying which asset this panel is displaying
		bool bShowAssetName;
	
		// Command list for this diff panel
		TSharedPtr<FUICommandList> GraphEditorCommands;

		// Property View 
		TSharedPtr<class IDetailsView> DetailsView;
	};

	// Type def for our Table item
	typedef TSharedPtr<struct FTreeDiffResultItem> FSharedDiffOnGraph;

	// Type def of our Table Type
	typedef SListView<FSharedDiffOnGraph > SListViewType;


	/**
	 * User clicks defaults view button to display defaults in remote diff tool
	 * @return If its been handled
	 */
	FReply OnOpenInDefaults();

	/**
	 * Generate list of differences
	 * @return The widget containing the Difference list
	 */
	TSharedRef<SWidget> GenerateDiffListWidget();

	/**
	 * Build up the Diff Source Array
	 */
	void BuildDiffSourceArray();

	/**
	 * Go to Next Difference
	 */
	void NextDiff();

	/**
	 * Go to Previous Difference
	 */
	void PrevDiff();

	/**
	 * Get current Index into the diff array
	 * @return The Index
	 */
	int32 GetCurrentDiffIndex();

	/** 
	 * Called when difference selection is changed
	 * @param Item The item that was selected
	 * @param SelectionType The way it was selected
	 */
	void OnSelectionChanged(FSharedDiffOnGraph Item, ESelectInfo::Type SelectionType);

	/**
	 * Called when a new row is being generated
	 * @param Item The item being generated
	 * @param OwnerTable The table its going into
	 * @return The Row that was inserted
	 */
	TSharedRef<ITableRow> OnGenerateRow(FSharedDiffOnGraph Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** 
	 * Get the Slate graph editor of the supplied Graph
	 * @param Graph The graph we want the Editor from
	 * @return The Graph Editor
	 */
	SGraphEditor* GetGraphEditorForGraph(UEdGraph* Graph) const;

	/**
	 * Removes focus of the pin in the graph viewer
	 */
	void DisablePinDiffFocus();

private:

	// Delegate to call when user wishes to view the defaults
	FOpenInDefaults	 OpenInDefaults;

	// The 2 panels we will be comparing
	FBehaviorTreeDiffPanel PanelOld, PanelNew;

	// Source for list view 
	TArray<FSharedDiffOnGraph> DiffListSource;

	// result from FGraphDiffControl::DiffGraphs
	TSharedPtr<TArray<FDiffSingleResult>> FoundDiffs;

	// Key commands processed by this widget
	TSharedPtr<FUICommandList> KeyCommands;

	// ListView of differences
	TSharedPtr<SListViewType> DiffList;

	// The last pin the user clicked on
	UEdGraphPin* LastPinTarget;

	// The last other pin the user clicked on
	UEdGraphPin* LastOtherPinTarget;
};
