// Copyright Epic Games, Inc. All Rights Reserved.


#include "SGraphNode_BehaviorTree.h"
#include "Types/SlateStructs.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SToolTip.h"
#include "BehaviorTree/BTNode.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTreeGraph.h"
#include "BehaviorTreeGraphNode.h"
#include "BehaviorTreeGraphNode_Composite.h"
#include "BehaviorTreeGraphNode_CompositeDecorator.h"
#include "BehaviorTreeGraphNode_Decorator.h"
#include "BehaviorTreeGraphNode_Root.h"
#include "BehaviorTreeGraphNode_Service.h"
#include "BehaviorTreeGraphNode_Task.h"
#include "Editor.h"
#include "BehaviorTreeDebugger.h"
#include "GraphEditorSettings.h"
#include "SGraphPanel.h"
#include "SCommentBubble.h"
#include "SGraphPreviewer.h"
#include "NodeFactory.h"
#include "BehaviorTreeColors.h"
#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTree/Tasks/BTTask_RunBehavior.h"
#include "IDocumentation.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "SLevelOfDetailBranchNode.h"
#include "BehaviorTree/Decorators/BTDecorator_Blackboard.h"

#define LOCTEXT_NAMESPACE "BehaviorTreeEditor"

#define ALWAYS_SHOW_BT_EXECUTION_INDEX 1

bool ShouldShowExecutionIndex()
{
#ifdef ALWAYS_SHOW_BT_EXECUTION_INDEX
	return true;
#else
	return GEditor && (GEditor->bIsSimulatingInEditor || GEditor->PlayWorld != NULL);
#endif // ALWAYS_SHOW_BT_EXECUTION_INDEX
}

/////////////////////////////////////////////////////
// SBehaviorTreePin

class SBehaviorTreePin : public SGraphPinAI
{
public:
	SLATE_BEGIN_ARGS(SBehaviorTreePin){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin);
protected:
	/** @return The color that we should use to draw this pin */
	virtual FSlateColor GetPinColor() const override;
};

void SBehaviorTreePin::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	SGraphPinAI::Construct(SGraphPinAI::FArguments(), InPin);
}

FSlateColor SBehaviorTreePin::GetPinColor() const
{
	return 
		bIsDiffHighlighted ? BehaviorTreeColors::Pin::Diff :
		IsHovered() ? BehaviorTreeColors::Pin::Hover :
		(GraphPinObj->PinType.PinCategory == UBehaviorTreeEditorTypes::PinCategory_SingleComposite) ? BehaviorTreeColors::Pin::CompositeOnly :
		(GraphPinObj->PinType.PinCategory == UBehaviorTreeEditorTypes::PinCategory_SingleTask) ? BehaviorTreeColors::Pin::TaskOnly :
		(GraphPinObj->PinType.PinCategory == UBehaviorTreeEditorTypes::PinCategory_SingleNode) ? BehaviorTreeColors::Pin::SingleNode :
		BehaviorTreeColors::Pin::Default;
}

/** Widget for overlaying an execution-order index onto a node */
class SBehaviorTreeIndex : public SCompoundWidget
{
public:
	/** Delegate event fired when the hover state of this widget changes */
	DECLARE_DELEGATE_OneParam(FOnHoverStateChanged, bool /* bHovered */);

	/** Delegate used to receive the color of the node, depending on hover state and state of other siblings */
	DECLARE_DELEGATE_RetVal_OneParam(FSlateColor, FOnGetIndexColor, bool /* bHovered */);

	SLATE_BEGIN_ARGS(SBehaviorTreeIndex){}
		SLATE_ATTRIBUTE(FText, Text)
		SLATE_EVENT(FOnHoverStateChanged, OnHoverStateChanged)
		SLATE_EVENT(FOnGetIndexColor, OnGetIndexColor)
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		OnHoverStateChangedEvent = InArgs._OnHoverStateChanged;
		OnGetIndexColorEvent = InArgs._OnGetIndexColor;

		const FSlateBrush* IndexBrush = FAppStyle::GetBrush(TEXT("BTEditor.Graph.BTNode.Index"));

		ChildSlot
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				// Add a dummy box here to make sure the widget doesnt get smaller than the brush
				SNew(SBox)
				.WidthOverride(IndexBrush->ImageSize.X)
				.HeightOverride(IndexBrush->ImageSize.Y)
			]
			+SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SBorder)
				.BorderImage(IndexBrush)
				.BorderBackgroundColor(this, &SBehaviorTreeIndex::GetColor)
				.Padding(FMargin(4.0f, 0.0f, 4.0f, 1.0f))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(InArgs._Text)
					.Font(FAppStyle::GetFontStyle("BTEditor.Graph.BTNode.IndexText"))
				]
			]
		];
	}

	virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		OnHoverStateChangedEvent.ExecuteIfBound(true);
		SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
	}

	virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override
	{
		OnHoverStateChangedEvent.ExecuteIfBound(false);
		SCompoundWidget::OnMouseLeave(MouseEvent);
	}

	/** Get the color we use to display the rounded border */
	FSlateColor GetColor() const
	{
		if(OnGetIndexColorEvent.IsBound())
		{
			return OnGetIndexColorEvent.Execute(IsHovered());
		}

		return FSlateColor::UseForeground();
	}

private:
	/** Delegate event fired when the hover state of this widget changes */
	FOnHoverStateChanged OnHoverStateChangedEvent;

	/** Delegate used to receive the color of the node, depending on hover state and state of other siblings */
	FOnGetIndexColor OnGetIndexColorEvent;
};

/////////////////////////////////////////////////////
// SGraphNode_BehaviorTree

void SGraphNode_BehaviorTree::Construct(const FArguments& InArgs, UBehaviorTreeGraphNode* InNode)
{
	DebuggerStateDuration = 0.0f;
	DebuggerStateCounter = INDEX_NONE;
	bSuppressDebuggerTriggers = false;

	SGraphNodeAI::Construct(SGraphNodeAI::FArguments(), InNode);
}

void SGraphNode_BehaviorTree::AddDecorator(TSharedPtr<SGraphNode> DecoratorWidget)
{
	DecoratorsBox->AddSlot().AutoHeight()
	[
		DecoratorWidget.ToSharedRef()
	];

	DecoratorWidgets.Add(DecoratorWidget);
	AddSubNode(DecoratorWidget);
}

void SGraphNode_BehaviorTree::AddService(TSharedPtr<SGraphNode> ServiceWidget)
{
	ServicesBox->AddSlot().AutoHeight()
	[
		ServiceWidget.ToSharedRef()
	];
	ServicesWidgets.Add(ServiceWidget);
	AddSubNode(ServiceWidget);
}

FSlateColor SGraphNode_BehaviorTree::GetBorderBackgroundColor() const
{
	UBehaviorTreeGraphNode* BTGraphNode = Cast<UBehaviorTreeGraphNode>(GraphNode);
	UBehaviorTreeGraphNode* BTParentNode = BTGraphNode ? Cast<UBehaviorTreeGraphNode>(BTGraphNode->ParentNode) : nullptr;
	const bool bIsInDebuggerActiveState = BTGraphNode && BTGraphNode->bDebuggerMarkCurrentlyActive;
	const bool bIsInDebuggerPrevState = BTGraphNode && BTGraphNode->bDebuggerMarkPreviouslyActive;
	const bool bSelectedSubNode = BTParentNode && GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode);
	
	UBTNode* NodeInstance = BTGraphNode ? Cast<UBTNode>(BTGraphNode->NodeInstance) : NULL;
	const bool bIsConnectedTreeRoot = BTGraphNode && BTGraphNode->IsA<UBehaviorTreeGraphNode_Root>() && BTGraphNode->Pins.IsValidIndex(0) && BTGraphNode->Pins[0]->LinkedTo.Num() > 0;
	const bool bIsDisconnected = NodeInstance && NodeInstance->GetExecutionIndex() == MAX_uint16;
	const bool bIsService = BTGraphNode && BTGraphNode->IsA(UBehaviorTreeGraphNode_Service::StaticClass());
	const bool bIsRootDecorator = BTGraphNode && BTGraphNode->bRootLevel;
	const bool bIsInjected = BTGraphNode && BTGraphNode->bInjectedNode;
	const bool bIsBrokenWithParent = bIsService ? 
		BTParentNode && BTParentNode->Services.Find(BTGraphNode) == INDEX_NONE ? true : false :
		BTParentNode && BTParentNode->Decorators.Find(BTGraphNode) == INDEX_NONE ? true :
		(BTGraphNode && BTGraphNode->NodeInstance != NULL && (Cast<UBTNode>(BTGraphNode->NodeInstance->GetOuter()) == NULL && Cast<UBehaviorTree>(BTGraphNode->NodeInstance->GetOuter()) == NULL)) ? true : false;

	if (FBehaviorTreeDebugger::IsPIENotSimulating() && BTGraphNode)
	{
		if (BTGraphNode->bHighlightInAbortRange0)
		{
			return BehaviorTreeColors::NodeBorder::HighlightAbortRange0;
		}
		else if (BTGraphNode->bHighlightInAbortRange1)
		{
			return BehaviorTreeColors::NodeBorder::HighlightAbortRange1;
		}
		else if (BTGraphNode->bHighlightInSearchTree)
		{
			return BehaviorTreeColors::NodeBorder::QuickFind;
		}
	}

	return bSelectedSubNode ? BehaviorTreeColors::NodeBorder::Selected : 
		!bIsRootDecorator && !bIsInjected && bIsBrokenWithParent ? BehaviorTreeColors::NodeBorder::BrokenWithParent :
		!bIsRootDecorator && !bIsInjected && bIsDisconnected ? BehaviorTreeColors::NodeBorder::Disconnected :
		bIsInDebuggerActiveState ? BehaviorTreeColors::NodeBorder::ActiveDebugging :
		bIsInDebuggerPrevState ? BehaviorTreeColors::NodeBorder::InactiveDebugging :
		bIsConnectedTreeRoot ? BehaviorTreeColors::NodeBorder::Root :
		BehaviorTreeColors::NodeBorder::Inactive;
}

FSlateColor SGraphNode_BehaviorTree::GetBackgroundColor() const
{
	UBehaviorTreeGraphNode* BTGraphNode = Cast<UBehaviorTreeGraphNode>(GraphNode);
	const bool bIsActiveForDebugger = BTGraphNode ?
		!bSuppressDebuggerColor && (BTGraphNode->bDebuggerMarkCurrentlyActive || BTGraphNode->bDebuggerMarkPreviouslyActive) :
		false;

	FLinearColor NodeColor = BehaviorTreeColors::NodeBody::Default;
	if (BTGraphNode && BTGraphNode->HasErrors())
	{
		NodeColor = BehaviorTreeColors::NodeBody::Error;
	}
	else if (BTGraphNode)
	{
		NodeColor = BTGraphNode->GetBackgroundColor(bIsActiveForDebugger);
	}

	return (FlashAlpha > 0.0f) ? FMath::Lerp(NodeColor, FlashColor, FlashAlpha) : NodeColor;
}

void SGraphNode_BehaviorTree::UpdateGraphNode()
{
	bDragMarkerVisible = false;
	InputPins.Empty();
	OutputPins.Empty();

	if (DecoratorsBox.IsValid())
	{
		DecoratorsBox->ClearChildren();
	} 
	else
	{
		SAssignNew(DecoratorsBox,SVerticalBox);
	}

	if (ServicesBox.IsValid())
	{
		ServicesBox->ClearChildren();
	}
	else
	{
		SAssignNew(ServicesBox,SVerticalBox);
	}

	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();
	DecoratorWidgets.Reset();
	ServicesWidgets.Reset();
	SubNodes.Reset();
	OutputPinBox.Reset();

	UBehaviorTreeGraphNode* BTNode = Cast<UBehaviorTreeGraphNode>(GraphNode);

	if (BTNode)
	{
		for (int32 i = 0; i < BTNode->Decorators.Num(); i++)
		{
			if (BTNode->Decorators[i])
			{
				TSharedPtr<SGraphNode> NewNode = FNodeFactory::CreateNodeWidget(BTNode->Decorators[i]);
				if (OwnerGraphPanelPtr.IsValid())
				{
					NewNode->SetOwner(OwnerGraphPanelPtr.Pin().ToSharedRef());
					OwnerGraphPanelPtr.Pin()->AttachGraphEvents(NewNode);
				}
				AddDecorator(NewNode);
				NewNode->UpdateGraphNode();
			}
		}

		for (int32 i = 0; i < BTNode->Services.Num(); i++)
		{
			if (BTNode->Services[i])
			{
				TSharedPtr<SGraphNode> NewNode = FNodeFactory::CreateNodeWidget(BTNode->Services[i]);
				if (OwnerGraphPanelPtr.IsValid())
				{
					NewNode->SetOwner(OwnerGraphPanelPtr.Pin().ToSharedRef());
					OwnerGraphPanelPtr.Pin()->AttachGraphEvents(NewNode);
				}
				AddService(NewNode);
				NewNode->UpdateGraphNode();
			}
		}
	}

	TSharedPtr<SErrorText> ErrorText;
	TSharedPtr<STextBlock> DescriptionText; 
	TSharedPtr<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode);

	TWeakPtr<SNodeTitle> WeakNodeTitle = NodeTitle;
	auto GetNodeTitlePlaceholderWidth = [WeakNodeTitle]() -> FOptionalSize
	{
		TSharedPtr<SNodeTitle> NodeTitlePin = WeakNodeTitle.Pin();
		const float DesiredWidth = (NodeTitlePin.IsValid()) ? NodeTitlePin->GetTitleSize().X : 0.0f;
		return FMath::Max(75.0f, DesiredWidth);
	};
	auto GetNodeTitlePlaceholderHeight = [WeakNodeTitle]() -> FOptionalSize
	{
		TSharedPtr<SNodeTitle> NodeTitlePin = WeakNodeTitle.Pin();
		const float DesiredHeight = (NodeTitlePin.IsValid()) ? NodeTitlePin->GetTitleSize().Y : 0.0f;
		return FMath::Max(22.0f, DesiredHeight);
	};

	const FMargin NodePadding = (Cast<UBehaviorTreeGraphNode_Decorator>(GraphNode) || Cast<UBehaviorTreeGraphNode_CompositeDecorator>(GraphNode) || Cast<UBehaviorTreeGraphNode_Service>(GraphNode))
		? FMargin(2.0f)
		: FMargin(8.0f);

	IndexOverlay = SNew(SBehaviorTreeIndex)
		.ToolTipText(this, &SGraphNode_BehaviorTree::GetIndexTooltipText)
		.Visibility(this, &SGraphNode_BehaviorTree::GetIndexVisibility)
		.Text(this, &SGraphNode_BehaviorTree::GetIndexText)
		.OnHoverStateChanged(this, &SGraphNode_BehaviorTree::OnIndexHoverStateChanged)
		.OnGetIndexColor(this, &SGraphNode_BehaviorTree::GetIndexColor);

	this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush( "Graph.StateNode.Body" ) )
			.Padding(0.0f)
			.BorderBackgroundColor( this, &SGraphNode_BehaviorTree::GetBorderBackgroundColor )
			.OnMouseButtonDown(this, &SGraphNode_BehaviorTree::OnMouseDown)
			[
				SNew(SOverlay)

				// Pins and node details
				+SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SVerticalBox)

					// INPUT PIN AREA
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBox)
						.MinDesiredHeight(NodePadding.Top)
						[
							SAssignNew(LeftNodeBox, SVerticalBox)
						]
					]

					// STATE NAME AREA
					+SVerticalBox::Slot()
					.Padding(FMargin(NodePadding.Left, 0.0f, NodePadding.Right, 0.0f))
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							DecoratorsBox.ToSharedRef()
						]
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							SAssignNew(NodeBody, SBorder)
							.BorderImage( FAppStyle::GetBrush("BTEditor.Graph.BTNode.Body") )
							.BorderBackgroundColor( this, &SGraphNode_BehaviorTree::GetBackgroundColor )
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Center)
							.Visibility(EVisibility::SelfHitTestInvisible)
							[
								SNew(SOverlay)
								+SOverlay::Slot()
								.HAlign(HAlign_Fill)
								.VAlign(VAlign_Fill)
								[
									SNew(SVerticalBox)
									+SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SHorizontalBox)
										+SHorizontalBox::Slot()
										.AutoWidth()
										[
											// POPUP ERROR MESSAGE
											SAssignNew(ErrorText, SErrorText )
											.BackgroundColor( this, &SGraphNode_BehaviorTree::GetErrorColor )
											.ToolTipText( this, &SGraphNode_BehaviorTree::GetErrorMsgToolTip )
										]
										
										+SHorizontalBox::Slot()
										.AutoWidth()
										[
											SNew(SLevelOfDetailBranchNode)
											.UseLowDetailSlot(this, &SGraphNode_BehaviorTree::UseLowDetailNodeTitles)
											.LowDetail()
											[
												SNew(SBox)
												.WidthOverride_Lambda(GetNodeTitlePlaceholderWidth)
												.HeightOverride_Lambda(GetNodeTitlePlaceholderHeight)
											]
											.HighDetail()
											[
												SNew(SHorizontalBox)
												+SHorizontalBox::Slot()
												.AutoWidth()
												.VAlign(VAlign_Center)
												[
													SNew(SImage)
													.Image(this, &SGraphNode_BehaviorTree::GetNameIcon)
												]
												+SHorizontalBox::Slot()
												.Padding(FMargin(4.0f, 0.0f, 4.0f, 0.0f))
												[
													SNew(SVerticalBox)
													+SVerticalBox::Slot()
													.AutoHeight()
													[
														SAssignNew(InlineEditableText, SInlineEditableTextBlock)
														.Style( FAppStyle::Get(), "Graph.StateNode.NodeTitleInlineEditableText" )
														.Text( NodeTitle.Get(), &SNodeTitle::GetHeadTitle )
														.OnVerifyTextChanged(this, &SGraphNode_BehaviorTree::OnVerifyNameTextChanged)
														.OnTextCommitted(this, &SGraphNode_BehaviorTree::OnNameTextCommited)
														.IsReadOnly( this, &SGraphNode_BehaviorTree::IsNameReadOnly )
														.IsSelected(this, &SGraphNode_BehaviorTree::IsSelectedExclusively)
													]
													+SVerticalBox::Slot()
													.AutoHeight()
													[
														NodeTitle.ToSharedRef()
													]
												]
											]
										]
									]
									+SVerticalBox::Slot()
									.AutoHeight()
									[
										// DESCRIPTION MESSAGE
										SAssignNew(DescriptionText, STextBlock )
										.Visibility(this, &SGraphNode_BehaviorTree::GetDescriptionVisibility)
										.Text(this, &SGraphNode_BehaviorTree::GetDescription)
									]
									+SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(STextBlock)
										.Visibility(this, &SGraphNode_BehaviorTree::GetRuntimeDescriptionVisibility)
										.Text(this, &SGraphNode_BehaviorTree::GetRuntimeDescription)
										.TextStyle(FAppStyle::Get(), "SmallText")
									]
								]
								+SOverlay::Slot()
								.HAlign(HAlign_Right)
								.VAlign(VAlign_Fill)
								[
									SNew(SBorder)
									.BorderImage( FAppStyle::GetBrush("BTEditor.Graph.BTNode.Body") )
									.BorderBackgroundColor(BehaviorTreeColors::Debugger::SearchFailed)
									.Padding(FMargin(4.0f, 0.0f))
									.Visibility(this, &SGraphNode_BehaviorTree::GetDebuggerSearchFailedMarkerVisibility)
								]
							]
						]
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(FMargin(10.0f,0,0,0))
						[
							ServicesBox.ToSharedRef()
						]
					]

					// OUTPUT PIN AREA
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBox)
						.MinDesiredHeight(NodePadding.Bottom)
						[
							SAssignNew(RightNodeBox, SVerticalBox)
							+SVerticalBox::Slot()
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Fill)
							.Padding(20.0f,0.0f)
							.FillHeight(1.0f)
							[
								SAssignNew(OutputPinBox, SHorizontalBox)
							]
						]
					]
				]

				// Drag marker overlay
				+SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				[
					SNew(SBorder)
					.BorderBackgroundColor(BehaviorTreeColors::Action::DragMarker)
					.ColorAndOpacity(BehaviorTreeColors::Action::DragMarker)
					.BorderImage(FAppStyle::GetBrush("BTEditor.Graph.BTNode.Body"))
					.Visibility(this, &SGraphNode_BehaviorTree::GetDragOverMarkerVisibility)
					[
						SNew(SBox)
						.HeightOverride(4.f)
					]
				]

				// Blueprint indicator overlay
				+SOverlay::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("BTEditor.Graph.BTNode.Blueprint")))
					.Visibility(this, &SGraphNode_BehaviorTree::GetBlueprintIconVisibility)
				]
			]
		];
	// Create comment bubble
	TSharedPtr<SCommentBubble> CommentBubble;
	const FSlateColor CommentColor = GetDefault<UGraphEditorSettings>()->DefaultCommentNodeTitleColor;

	SAssignNew( CommentBubble, SCommentBubble )
	.GraphNode( GraphNode )
	.Text( this, &SGraphNode::GetNodeComment )
	.OnTextCommitted( this, &SGraphNode::OnCommentTextCommitted )
	.ColorAndOpacity( CommentColor )
	.AllowPinning( true )
	.EnableTitleBarBubble( true )
	.EnableBubbleCtrls( true )
	.GraphLOD( this, &SGraphNode::GetCurrentLOD )
	.IsGraphNodeHovered( this, &SGraphNode::IsHovered );

	GetOrAddSlot( ENodeZone::TopCenter )
	.SlotOffset2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetOffset2f ))
	.SlotSize2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetSize2f ))
	.AllowScaling( TAttribute<bool>( CommentBubble.Get(), &SCommentBubble::IsScalingAllowed ))
	.VAlign( VAlign_Top )
	[
		CommentBubble.ToSharedRef()
	];

	ErrorReporting = ErrorText;
	ErrorReporting->SetError(ErrorMsg);
	CreatePinWidgets();
}

EVisibility SGraphNode_BehaviorTree::GetDebuggerSearchFailedMarkerVisibility() const
{
	UBehaviorTreeGraphNode_Decorator* MyNode = Cast<UBehaviorTreeGraphNode_Decorator>(GraphNode);
	return MyNode && MyNode->bDebuggerMarkSearchFailed ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
}

void SGraphNode_BehaviorTree::PreChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType)
{
	// Implementing interface pure virtual method but nothing to do here
}

void SGraphNode_BehaviorTree::PostChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType)
{
	if (const UBehaviorTreeGraphNode_Decorator* DecoratorNode = Cast<UBehaviorTreeGraphNode_Decorator>(GraphNode))
	{
		if (UBTDecorator_Blackboard* Decorator = Cast<UBTDecorator_Blackboard>(DecoratorNode->NodeInstance))
		{
			Decorator->BuildDescription();
		}
	}
	
	if (UBehaviorTreeGraphNode_CompositeDecorator* DecoratorNode = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(GraphNode))
	{
		DecoratorNode->BuildDescription();
	}
}

void SGraphNode_BehaviorTree::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SGraphNode::Tick( AllottedGeometry, InCurrentTime, InDeltaTime );
	CachedPosition = AllottedGeometry.AbsolutePosition / AllottedGeometry.Scale;

	UBehaviorTreeGraphNode* MyNode = Cast<UBehaviorTreeGraphNode>(GraphNode);
	if (MyNode && MyNode->DebuggerUpdateCounter != DebuggerStateCounter)
	{
		DebuggerStateCounter = MyNode->DebuggerUpdateCounter;
		DebuggerStateDuration = 0.0f;
		bSuppressDebuggerColor = false;
		bSuppressDebuggerTriggers = false;
	}

	DebuggerStateDuration += InDeltaTime;

	UBehaviorTreeGraphNode* BTGraphNode = Cast<UBehaviorTreeGraphNode>(GraphNode);
	float NewFlashAlpha = 0.0f;
	TriggerOffsets.Reset();

	if (BTGraphNode && FBehaviorTreeDebugger::IsPlaySessionPaused())
	{
		const float SearchPathDelay = 0.5f;
		const float SearchPathBlink = 1.0f;
		const float SearchPathBlinkFreq = 10.0f;
		const float SearchPathKeepTime = 2.0f;
		const float ActiveFlashDuration = 0.2f;

		const bool bHasResult = BTGraphNode->bDebuggerMarkSearchSucceeded || BTGraphNode->bDebuggerMarkSearchFailed;
		const bool bHasTriggers = !bSuppressDebuggerTriggers && (BTGraphNode->bDebuggerMarkSearchTrigger || BTGraphNode->bDebuggerMarkSearchFailedTrigger);
		if (bHasResult || bHasTriggers)
		{
			const float FlashStartTime = BTGraphNode->DebuggerSearchPathIndex * SearchPathDelay;
			const float FlashStopTime = (BTGraphNode->DebuggerSearchPathSize * SearchPathDelay) + SearchPathKeepTime;
			
			UBehaviorTreeGraphNode_Decorator* BTGraph_Decorator = Cast<UBehaviorTreeGraphNode_Decorator>(GraphNode);
			UBehaviorTreeGraphNode_CompositeDecorator* BTGraph_CompDecorator = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(GraphNode);

			bSuppressDebuggerColor = (DebuggerStateDuration < FlashStopTime);
			if (bSuppressDebuggerColor)
			{
				if (bHasResult && (BTGraph_Decorator || BTGraph_CompDecorator))
				{
					NewFlashAlpha =
						(DebuggerStateDuration > FlashStartTime + SearchPathBlink) ? 1.0f :
						(FMath::TruncToInt(DebuggerStateDuration * SearchPathBlinkFreq) % 2) ? 1.0f : 0.0f;
				} 
			}

			FlashColor = BTGraphNode->bDebuggerMarkSearchSucceeded ?
				BehaviorTreeColors::Debugger::SearchSucceeded :
				BehaviorTreeColors::Debugger::SearchFailed;
		}
		else if (BTGraphNode->bDebuggerMarkFlashActive)
		{
			NewFlashAlpha = (DebuggerStateDuration < ActiveFlashDuration) ?
				FMath::Square(1.0f - (DebuggerStateDuration / ActiveFlashDuration)) : 
				0.0f;

			FlashColor = BehaviorTreeColors::Debugger::TaskFlash;
		}

		if (bHasTriggers)
		{
			// find decorator that caused restart
			for (int32 i = 0; i < DecoratorWidgets.Num(); i++)
			{
				if (DecoratorWidgets[i].IsValid())
				{
					SGraphNode_BehaviorTree* TestSNode = (SGraphNode_BehaviorTree*)DecoratorWidgets[i].Get();
					UBehaviorTreeGraphNode* ChildNode = Cast<UBehaviorTreeGraphNode>(TestSNode->GraphNode);
					if (ChildNode && (ChildNode->bDebuggerMarkSearchFailedTrigger || ChildNode->bDebuggerMarkSearchTrigger))
					{
						TriggerOffsets.Add(FNodeBounds(TestSNode->GetCachedPosition() - CachedPosition, TestSNode->GetDesiredSize()));
					}
				}
			}

			// when it wasn't any of them, add node itself to triggers (e.g. parallel's main task)
			if (DecoratorWidgets.Num() == 0)
			{
				TriggerOffsets.Add(FNodeBounds(FVector2f(0.f,0.f),GetDesiredSize()));
			}
		}
	}
	FlashAlpha = NewFlashAlpha;
}

FReply SGraphNode_BehaviorTree::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	return SGraphNode::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent );
}

FText SGraphNode_BehaviorTree::GetPinTooltip(UEdGraphPin* GraphPinObj) const
{
	FText HoverText = FText::GetEmpty();

	check(GraphPinObj != nullptr);
	UEdGraphNode* OwningGraphNode = GraphPinObj->GetOwningNode();
	if (OwningGraphNode != nullptr)
	{
		FString HoverStr;
		OwningGraphNode->GetPinHoverText(*GraphPinObj, /*out*/HoverStr);
		if (!HoverStr.IsEmpty())
		{
			HoverText = FText::FromString(HoverStr);
		}
	}

	return HoverText;
}

FText SGraphNode_BehaviorTree::GetRuntimeDescription() const
{
	const UBehaviorTreeGraphNode* MyNode = CastChecked<UBehaviorTreeGraphNode>(GraphNode);
	return MyNode ? FText::FromString(MyNode->DebuggerRuntimeDescription) : FText::GetEmpty();
}

EVisibility SGraphNode_BehaviorTree::GetRuntimeDescriptionVisibility() const
{
	const UBehaviorTreeGraphNode* MyNode = CastChecked<UBehaviorTreeGraphNode>(GraphNode);

	return GetDescriptionVisibility().IsVisible() && !MyNode->DebuggerRuntimeDescription.IsEmpty()
		? EVisibility::HitTestInvisible
		: EVisibility::Collapsed;
}

void SGraphNode_BehaviorTree::CreatePinWidgets()
{
	UBehaviorTreeGraphNode* StateNode = CastChecked<UBehaviorTreeGraphNode>(GraphNode);

	for (int32 PinIdx = 0; PinIdx < StateNode->Pins.Num(); PinIdx++)
	{
		UEdGraphPin* MyPin = StateNode->Pins[PinIdx];
		if (!MyPin->bHidden)
		{
			TSharedPtr<SGraphPin> NewPin = SNew(SBehaviorTreePin, MyPin)
				.ToolTipText( this, &SGraphNode_BehaviorTree::GetPinTooltip, MyPin);

			AddPin(NewPin.ToSharedRef());
		}
	}
}

void SGraphNode_BehaviorTree::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	PinToAdd->SetOwner( SharedThis(this) );

	const UEdGraphPin* PinObj = PinToAdd->GetPinObj();
	const bool bAdvancedParameter = PinObj && PinObj->bAdvancedView;
	if (bAdvancedParameter)
	{
		PinToAdd->SetVisibility( TAttribute<EVisibility>(PinToAdd, &SGraphPin::IsPinVisibleAsAdvanced) );
	}

	if (PinToAdd->GetDirection() == EEdGraphPinDirection::EGPD_Input)
	{
		LeftNodeBox->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillHeight(1.0f)
			.Padding(20.0f,0.0f)
			[
				PinToAdd
			];
		InputPins.Add(PinToAdd);
	}
	else // Direction == EEdGraphPinDirection::EGPD_Output
	{
		const bool bIsSingleTaskPin = PinObj && (PinObj->PinType.PinCategory == UBehaviorTreeEditorTypes::PinCategory_SingleTask);
		if (bIsSingleTaskPin)
		{
			OutputPinBox->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillWidth(0.4f)
			.Padding(0,0,20.0f,0)
			[
				PinToAdd
			];
		}
		else
		{
			OutputPinBox->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillWidth(1.0f)
			[
				PinToAdd
			];
		}
		OutputPins.Add(PinToAdd);
	}
}

TSharedPtr<SToolTip> SGraphNode_BehaviorTree::GetComplexTooltip()
{
	const UBehaviorTreeGraphNode* BTGraphNode = Cast<UBehaviorTreeGraphNode>(GraphNode);
	const bool bHasErrors = BTGraphNode && BTGraphNode->HasErrors();

	if (!bHasErrors)
	{
		UBehaviorTreeGraphNode_CompositeDecorator* DecoratorNode = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(GraphNode);
		if (DecoratorNode && DecoratorNode->GetBoundGraph())
		{
			return SNew(SToolTip)
				[
					SNew(SOverlay)
					+SOverlay::Slot()
					[
						// Create the tooltip graph preview, make sure to disable state overlays to
						// prevent the PIE / read-only borders from obscuring the graph
						SNew(SGraphPreviewer, DecoratorNode->GetBoundGraph())
						.CornerOverlayText(LOCTEXT("CompositeDecoratorOverlayText", "Composite Decorator"))
						.ShowGraphStateOverlay(false)
					]
					+SOverlay::Slot()
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CompositeDecoratorTooltip", "Double-click to Open"))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					]
				];
		}

		UBehaviorTreeGraphNode_Task* TaskNode = Cast<UBehaviorTreeGraphNode_Task>(GraphNode);
		if(TaskNode && TaskNode->NodeInstance)
		{
			UBTTask_RunBehavior* RunBehavior = Cast<UBTTask_RunBehavior>(TaskNode->NodeInstance);
			if(RunBehavior && RunBehavior->GetSubtreeAsset() && RunBehavior->GetSubtreeAsset()->BTGraph)
			{
				return SNew(SToolTip)
					[
						SNew(SOverlay)
						+SOverlay::Slot()
						[
							// Create the tooltip graph preview, make sure to disable state overlays to
							// prevent the PIE / read-only borders from obscuring the graph
							SNew(SGraphPreviewer, RunBehavior->GetSubtreeAsset()->BTGraph)
							.CornerOverlayText(LOCTEXT("RunBehaviorOverlayText", "Run Behavior"))
							.ShowGraphStateOverlay(false)
						]
						+SOverlay::Slot()
						.Padding(2.0f)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("RunBehaviorTooltip", "Double-click to Open"))
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						]
					];
			}
		}
	}

	return IDocumentation::Get()->CreateToolTip(TAttribute<FText>(this, &SGraphNode::GetNodeTooltip), NULL, GraphNode->GetDocumentationLink(), GraphNode->GetDocumentationExcerptName());
}

const FSlateBrush* SGraphNode_BehaviorTree::GetNameIcon() const
{	
	UBehaviorTreeGraphNode* BTGraphNode = Cast<UBehaviorTreeGraphNode>(GraphNode);
	if (BTGraphNode != nullptr)
	{
		return BTGraphNode->GetNameIconStyleSet().GetBrush(BTGraphNode->GetNameIcon());
	}

	return FAppStyle::GetBrush(TEXT("BTEditor.Graph.BTNode.Icon"));
}

static UBehaviorTreeGraphNode* GetParentNode(UEdGraphNode* GraphNode)
{
	UBehaviorTreeGraphNode* BTGraphNode = Cast<UBehaviorTreeGraphNode>(GraphNode);
	if (BTGraphNode->ParentNode != nullptr)
	{
		BTGraphNode = Cast<UBehaviorTreeGraphNode>(BTGraphNode->ParentNode);
	}

	UEdGraphPin* MyInputPin = BTGraphNode->GetInputPin();
	UEdGraphPin* MyParentOutputPin = nullptr;
	if (MyInputPin != nullptr && MyInputPin->LinkedTo.Num() > 0)
	{
		MyParentOutputPin = MyInputPin->LinkedTo[0];
		if(MyParentOutputPin != nullptr)
		{
			if(MyParentOutputPin->GetOwningNode() != nullptr)
			{
				return CastChecked<UBehaviorTreeGraphNode>(MyParentOutputPin->GetOwningNode());
			}
		}
	}

	return nullptr;
}

void SGraphNode_BehaviorTree::OnIndexHoverStateChanged(bool bHovered)
{
	UBehaviorTreeGraphNode* ParentNode = GetParentNode(GraphNode);
	if(ParentNode != nullptr)
	{
		ParentNode->bHighlightChildNodeIndices = bHovered;
	}
}

FSlateColor SGraphNode_BehaviorTree::GetIndexColor(bool bHovered) const
{
	UBehaviorTreeGraphNode* ParentNode = GetParentNode(GraphNode);
	const bool bHighlightHover = bHovered || (ParentNode && ParentNode->bHighlightChildNodeIndices);

	static const FName HoveredColor("BTEditor.Graph.BTNode.Index.HoveredColor");
	static const FName DefaultColor("BTEditor.Graph.BTNode.Index.Color");

	return bHighlightHover ? FAppStyle::Get().GetSlateColor(HoveredColor) : FAppStyle::Get().GetSlateColor(DefaultColor);
}

EVisibility SGraphNode_BehaviorTree::GetIndexVisibility() const
{
	// always hide the index on the root node
	if(GraphNode->IsA(UBehaviorTreeGraphNode_Root::StaticClass()))
	{
		return EVisibility::Collapsed;
	}

	UBehaviorTreeGraphNode* StateNode = CastChecked<UBehaviorTreeGraphNode>(GraphNode);
	UEdGraphPin* MyInputPin = StateNode->GetInputPin();
	UEdGraphPin* MyParentOutputPin = NULL;
	if (MyInputPin != NULL && MyInputPin->LinkedTo.Num() > 0)
	{
		MyParentOutputPin = MyInputPin->LinkedTo[0];
	}

	// Visible if we are in PIE or if we have siblings
	const bool bCanShowIndex = ShouldShowExecutionIndex() || (MyParentOutputPin && MyParentOutputPin->LinkedTo.Num() > 1);

	// LOD this out once things get too small
	TSharedPtr<SGraphPanel> MyOwnerPanel = GetOwnerPanel();
	return (bCanShowIndex && (!MyOwnerPanel.IsValid() || MyOwnerPanel->GetCurrentLOD() > EGraphRenderingLOD::LowDetail)) ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SGraphNode_BehaviorTree::GetIndexText() const
{
	UBehaviorTreeGraphNode* StateNode = CastChecked<UBehaviorTreeGraphNode>(GraphNode);
	UEdGraphPin* MyInputPin = StateNode->GetInputPin();
	UEdGraphPin* MyParentOutputPin = NULL;
	if (MyInputPin != NULL && MyInputPin->LinkedTo.Num() > 0)
	{
		MyParentOutputPin = MyInputPin->LinkedTo[0];
	}

	int32 Index = 0;

	if (ShouldShowExecutionIndex())
	{
		// special case: range of execution indices in composite decorator node
		UBehaviorTreeGraphNode_CompositeDecorator* CompDecorator = Cast<UBehaviorTreeGraphNode_CompositeDecorator>(GraphNode);
		if (CompDecorator && CompDecorator->FirstExecutionIndex != CompDecorator->LastExecutionIndex)
		{
			return FText::Format(LOCTEXT("CompositeDecoratorFormat", "{0}..{1}"), FText::AsNumber(CompDecorator->FirstExecutionIndex), FText::AsNumber(CompDecorator->LastExecutionIndex));
		}

		// show execution index (debugging purposes)
		UBTNode* BTNode = Cast<UBTNode>(StateNode->NodeInstance);
		Index = (BTNode && BTNode->GetExecutionIndex() < 0xffff) ? BTNode->GetExecutionIndex() : -1;
	}
	else
	{
		// show child index
		if (MyParentOutputPin != NULL)
		{
			for (Index = 0; Index < MyParentOutputPin->LinkedTo.Num(); ++Index)
			{
				if (MyParentOutputPin->LinkedTo[Index] == MyInputPin)
				{
					break;
				}
			}
		}
	}

	return FText::AsNumber(Index);
}

FText SGraphNode_BehaviorTree::GetIndexTooltipText() const
{
	if (ShouldShowExecutionIndex())
	{
		return LOCTEXT("ExecutionIndexTooltip", "Execution index: this shows the order in which nodes are executed.");
	}
	else
	{
		return LOCTEXT("ChildIndexTooltip", "Child index: this shows the order in which child nodes are executed.");
	}
}

EVisibility SGraphNode_BehaviorTree::GetBlueprintIconVisibility() const
{
	UBehaviorTreeGraphNode* BTGraphNode = Cast<UBehaviorTreeGraphNode>(GraphNode);
	const bool bCanShowIcon = (BTGraphNode != nullptr && BTGraphNode->UsesBlueprint());

	// LOD this out once things get too small
	TSharedPtr<SGraphPanel> MyOwnerPanel = GetOwnerPanel();
	return (bCanShowIcon && (!MyOwnerPanel.IsValid() || MyOwnerPanel->GetCurrentLOD() > EGraphRenderingLOD::LowDetail)) ? EVisibility::Visible : EVisibility::Collapsed;
}

void SGraphNode_BehaviorTree::GetOverlayBrushes(bool bSelected, const FVector2f& WidgetSize, TArray<FOverlayBrushInfo>& Brushes) const
{
	UBehaviorTreeGraphNode* BTNode = Cast<UBehaviorTreeGraphNode>(GraphNode);
	if (BTNode == NULL)
	{
		return;
	}

	if (BTNode->bHasBreakpoint)
	{
		FOverlayBrushInfo BreakpointOverlayInfo;
		BreakpointOverlayInfo.Brush = BTNode->bIsBreakpointEnabled ?
			FAppStyle::GetBrush(TEXT("BTEditor.DebuggerOverlay.Breakpoint.Enabled")) :
			FAppStyle::GetBrush(TEXT("BTEditor.DebuggerOverlay.Breakpoint.Disabled"));

		if (BreakpointOverlayInfo.Brush)
		{
			BreakpointOverlayInfo.OverlayOffset -= BreakpointOverlayInfo.Brush->ImageSize / 2.f;
		}

		Brushes.Add(BreakpointOverlayInfo);
	}

	if (FBehaviorTreeDebugger::IsPlaySessionPaused())
	{
		if (BTNode->bDebuggerMarkBreakpointTrigger || (BTNode->bDebuggerMarkCurrentlyActive && BTNode->IsA(UBehaviorTreeGraphNode_Task::StaticClass())))
		{
			FOverlayBrushInfo IPOverlayInfo;

			IPOverlayInfo.Brush = BTNode->bDebuggerMarkBreakpointTrigger ? FAppStyle::GetBrush(TEXT("BTEditor.DebuggerOverlay.BreakOnBreakpointPointer")) : 
				FAppStyle::GetBrush(TEXT("BTEditor.DebuggerOverlay.ActiveNodePointer"));
			if (IPOverlayInfo.Brush)
			{
				float Overlap = 10.f;
				IPOverlayInfo.OverlayOffset.X = (WidgetSize.X/2.f) - (IPOverlayInfo.Brush->ImageSize.X/2.f);
				IPOverlayInfo.OverlayOffset.Y = (Overlap - IPOverlayInfo.Brush->ImageSize.Y);
			}

			IPOverlayInfo.AnimationEnvelope = FVector2f(0.f, 10.f);
			Brushes.Add(IPOverlayInfo);
		}

		if (TriggerOffsets.Num())
		{
			FOverlayBrushInfo IPOverlayInfo;

			IPOverlayInfo.Brush = FAppStyle::GetBrush(BTNode->bDebuggerMarkSearchTrigger ?
				TEXT("BTEditor.DebuggerOverlay.SearchTriggerPointer") :
				TEXT("BTEditor.DebuggerOverlay.FailedTriggerPointer") );

			if (IPOverlayInfo.Brush)
			{
				for (int32 i = 0; i < TriggerOffsets.Num(); i++)
				{
					IPOverlayInfo.OverlayOffset.X = -IPOverlayInfo.Brush->ImageSize.X;
					IPOverlayInfo.OverlayOffset.Y = TriggerOffsets[i].Position.Y + TriggerOffsets[i].Size.Y / 2.f - IPOverlayInfo.Brush->ImageSize.Y / 2.f;

					IPOverlayInfo.AnimationEnvelope = FVector2f(10.f, 0.f);
					Brushes.Add(IPOverlayInfo);
				}
			}
		}
	}
}

TArray<FOverlayWidgetInfo> SGraphNode_BehaviorTree::GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const
{
	TArray<FOverlayWidgetInfo> Widgets;

	check(NodeBody.IsValid());
	check(IndexOverlay.IsValid());

	FVector2f Origin(0.0f, 0.0f);

	// build overlays for decorator sub-nodes
	for(const auto& DecoratorWidget : DecoratorWidgets)
	{
		TArray<FOverlayWidgetInfo> OverlayWidgets = DecoratorWidget->GetOverlayWidgets(bSelected, WidgetSize);
		for(auto& OverlayWidget : OverlayWidgets)
		{
			OverlayWidget.OverlayOffset.Y += Origin.Y;
		}
		Widgets.Append(OverlayWidgets);
		Origin.Y += DecoratorWidget->GetDesiredSize().Y;
	}

	FOverlayWidgetInfo Overlay(IndexOverlay);
	Overlay.OverlayOffset = FVector2f(WidgetSize.X - (IndexOverlay->GetDesiredSize().X * 0.5f), Origin.Y);
	Widgets.Add(Overlay);

	Origin.Y += NodeBody->GetDesiredSize().Y;

	// build overlays for service sub-nodes
	for(const auto& ServiceWidget : ServicesWidgets)
	{
		TArray<FOverlayWidgetInfo> OverlayWidgets = ServiceWidget->GetOverlayWidgets(bSelected, WidgetSize);
		for(auto& OverlayWidget : OverlayWidgets)
		{
			OverlayWidget.OverlayOffset.Y += Origin.Y;
		}
		Widgets.Append(OverlayWidgets);
		Origin.Y += ServiceWidget->GetDesiredSize().Y;
	}

	return Widgets;
}

TSharedRef<SGraphNode> SGraphNode_BehaviorTree::GetNodeUnderMouse(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<SGraphNode> SubNode = GetSubNodeUnderCursor(MyGeometry, MouseEvent);
	return SubNode.IsValid() ? SubNode.ToSharedRef() : StaticCastSharedRef<SGraphNode>(AsShared());
}

void SGraphNode_BehaviorTree::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	SGraphNodeAI::MoveTo(NewPosition, NodeFilter, bMarkDirty);

	// keep node order (defined by linked pins) up to date with actual positions
	// this function will keep spamming on every mouse move update
	UBehaviorTreeGraphNode* BTGraphNode = Cast<UBehaviorTreeGraphNode>(GraphNode);
	if (BTGraphNode && !BTGraphNode->IsSubNode())
	{
		UBehaviorTreeGraph* BTGraph = BTGraphNode->GetOwnerBehaviorTreeGraph();
		if (BTGraph)
		{
			for (int32 Idx = 0; Idx < BTGraphNode->Pins.Num(); Idx++)
			{
				UEdGraphPin* Pin = BTGraphNode->Pins[Idx];
				if (Pin && Pin->Direction == EGPD_Input && Pin->LinkedTo.Num() == 1) 
				{
					UEdGraphPin* ParentPin = Pin->LinkedTo[0];
					if (ParentPin)
					{
						BTGraph->RebuildChildOrder(ParentPin->GetOwningNode());
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
