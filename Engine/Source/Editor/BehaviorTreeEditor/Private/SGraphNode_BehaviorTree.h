// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTreeEditorTypes.h"
#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Kismet2/EnumEditorUtils.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "SGraphNodeAI.h"
#include "SNodePanel.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Types/SlateVector2.h"

class SBorder;
class SGraphNode;
class SGraphPin;
class SHorizontalBox;
class SToolTip;
class SVerticalBox;
class SWidget;
class UBehaviorTreeGraphNode;
class UEdGraphPin;
class UUserDefinedEnum;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;

class SGraphNode_BehaviorTree : public SGraphNodeAI, public FEnumEditorUtils::INotifyOnEnumChanged
{
public:
	SLATE_BEGIN_ARGS(SGraphNode_BehaviorTree){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UBehaviorTreeGraphNode* InNode);

	// SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual void CreatePinWidgets() override;
	virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;
	virtual TSharedPtr<SToolTip> GetComplexTooltip() override;
	virtual void GetOverlayBrushes(bool bSelected, const FVector2f& WidgetSize, TArray<FOverlayBrushInfo>& Brushes) const override;
	virtual TArray<FOverlayWidgetInfo> GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const override;
	virtual TSharedRef<SGraphNode> GetNodeUnderMouse(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;
	// End of SGraphNode interface

	/** handle double click */
	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;

	/**
	 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	
	/** adds decorator widget inside current node */
	void AddDecorator(TSharedPtr<SGraphNode> DecoratorWidget);

	/** adds service widget inside current node */
	void AddService(TSharedPtr<SGraphNode> ServiceWidget);

	/** shows red marker when search failed*/
	EVisibility GetDebuggerSearchFailedMarkerVisibility() const;

	FVector2f GetCachedPosition() const { return CachedPosition; }

protected:
	/** INotifyOnEnumChanged interface */
	virtual void PreChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType) override;
	virtual void PostChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType) override;

	uint32 bSuppressDebuggerColor : 1;
	uint32 bSuppressDebuggerTriggers : 1;
	
	/** time spent in current state */
	float DebuggerStateDuration;

	/** currently displayed state */
	int32 DebuggerStateCounter;

	/** debugger colors */
	FLinearColor FlashColor;
	float FlashAlpha;

	/** height offsets for search triggers */
	TArray<FNodeBounds> TriggerOffsets;

	/** cached draw position */
	FVector2f CachedPosition;

	TArray< TSharedPtr<SGraphNode> > DecoratorWidgets;
	TArray< TSharedPtr<SGraphNode> > ServicesWidgets;
	TSharedPtr<SVerticalBox> DecoratorsBox;
	TSharedPtr<SVerticalBox> ServicesBox;
	TSharedPtr<SHorizontalBox> OutputPinBox;

	/** The widget we use to display the index of the node */
	TSharedPtr<SWidget> IndexOverlay;

	/** The node body widget, cached here so we can determine its size when we want ot position our overlays */
	TSharedPtr<SBorder> NodeBody;

	FSlateColor GetBorderBackgroundColor() const;
	FSlateColor GetBackgroundColor() const;

	virtual const FSlateBrush* GetNameIcon() const override;
	virtual EVisibility GetBlueprintIconVisibility() const;

	FText GetRuntimeDescription() const;
	EVisibility GetRuntimeDescriptionVisibility() const;

	/** Get the visibility of the index overlay */
	EVisibility GetIndexVisibility() const;

	/** Get the text to display in the index overlay */
	FText GetIndexText() const;

	/** Get the tooltip for the index overlay */
	FText GetIndexTooltipText() const;

	/** Get the color to display for the index overlay. This changes on hover state of sibling nodes */
	FSlateColor GetIndexColor(bool bHovered) const;

	/** Handle hover state changing for the index widget - we use this to highlight sibling nodes */
	void OnIndexHoverStateChanged(bool bHovered);

	FText GetPinTooltip(UEdGraphPin* GraphPinObj) const;
};
