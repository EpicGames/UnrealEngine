// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphNode_Decorator.h"

#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTreeDecoratorGraphNode_Decorator.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "Misc/Optional.h"
#include "Templates/Casts.h"

void SGraphNode_Decorator::Construct(const FArguments& InArgs, UBehaviorTreeDecoratorGraphNode_Decorator* InNode)
{
	this->GraphNode = InNode;

	this->SetCursor(EMouseCursor::CardinalCross);

	this->UpdateGraphNode();
}

FString SGraphNode_Decorator::GetNodeComment() const
{
	const UBehaviorTreeDecoratorGraphNode_Decorator* MyGraphNode = Cast<const UBehaviorTreeDecoratorGraphNode_Decorator>(GetNodeObj());
	const UBTDecorator* MyBTNode = MyGraphNode ? Cast<const UBTDecorator>(MyGraphNode->NodeInstance) : NULL;

	if (MyBTNode)
	{
		return MyBTNode->GetNodeName();
	}

	return SGraphNode::GetNodeComment();
}
