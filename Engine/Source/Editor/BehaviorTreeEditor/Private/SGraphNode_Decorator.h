// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "SGraphNode.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UBehaviorTreeDecoratorGraphNode_Decorator;

class SGraphNode_Decorator : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNode_Decorator){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UBehaviorTreeDecoratorGraphNode_Decorator* InNode);

	virtual FString GetNodeComment() const override;
};
