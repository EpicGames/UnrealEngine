// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "AssetDefinition_BehaviorTree.generated.h"

#define UE_API BEHAVIORTREEEDITOR_API

class UBehaviorTree;

UCLASS(MinimalAPI)
class UAssetDefinition_BehaviorTree : public UAssetDefinitionDefault
{
	GENERATED_BODY()

protected:
	// UAssetDefinition Begin
	UE_API virtual FText GetAssetDisplayName() const override;
	UE_API virtual FLinearColor GetAssetColor() const override;
	UE_API virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	UE_API virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	UE_API virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	UE_API virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	// UAssetDefinition End

private:

	/* Called to open the Behavior Tree defaults view, this opens whatever text diff tool the user has */
	UE_API void OpenInDefaults(class UBehaviorTree* OldBehaviorTree, class UBehaviorTree* NewBehaviorTree) const;
};

#undef UE_API
