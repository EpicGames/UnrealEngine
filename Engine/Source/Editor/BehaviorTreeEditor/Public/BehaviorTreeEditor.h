// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AIGraphEditor.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/IDelegateInstance.h"
#include "GraphEditor.h"
#include "HAL/PlatformMath.h"
#include "IBehaviorTreeEditor.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/NotifyHook.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkit.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API BEHAVIORTREEEDITOR_API

class FDocumentTabFactory;
class FDocumentTracker;
class FObjectPostSaveContext;
class FProperty;
class IDetailsView;
class SWidget;
class UBehaviorTree;
class UBehaviorTreeComponent;
class UBehaviorTreeGraph;
class UBlackboardData;
class UClass;
class UEdGraph;
class UEdGraphNode;
class UObject;
class UPackage;
struct FBlackboardEntry;
struct FPropertyChangedEvent;

class FBehaviorTreeEditor : public IBehaviorTreeEditor, public FAIGraphEditor, public FNotifyHook
{
public:
	UE_API FBehaviorTreeEditor();
	/** Destructor */
	UE_API virtual ~FBehaviorTreeEditor();

	UE_API virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	UE_API virtual void InitBehaviorTreeEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, UObject* InObject);

	//~ Begin IToolkit Interface
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;
	UE_API virtual FString GetWorldCentricTabPrefix() const override;
	UE_API virtual FLinearColor GetWorldCentricTabColorScale() const override;
	UE_API virtual FText GetToolkitName() const override;
	UE_API virtual FText GetToolkitToolTipText() const override;
	//~ End IToolkit Interface

	//~ Begin IBehaviorTreeEditor Interface
	UE_API virtual void InitializeDebuggerState(class FBehaviorTreeDebugger* ParentDebugger) const override;
	UE_API virtual UEdGraphNode* FindInjectedNode(int32 Index) const override;
	UE_API virtual void DoubleClickNode(class UEdGraphNode* Node) override;
	UE_API virtual void FocusAttentionOnNode(UEdGraphNode* Node) override;
	UE_API virtual void FocusWindow(UObject* ObjectToFocusOn = NULL) override;
	//~ End IBehaviorTreeEditor Interface

	//~ Begin IAssetEditorInstance Interface
	UE_API virtual bool IncludeAssetInRestoreOpenAssetsPrompt(UObject* Asset) const override;
	//~ End IAssetEditorInstance Interface

	//~ Begin FEditorUndoClient Interface
	UE_API virtual void PostUndo(bool bSuccess) override;
	UE_API virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

	//~ Begin FNotifyHook Interface
	UE_API virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged ) override;
	// End of FNotifyHook

	// Delegates
	UE_API void OnNodeDoubleClicked(class UEdGraphNode* Node);
	UE_API void OnGraphEditorFocused(const TSharedRef<SGraphEditor>& InGraphEditor);
	UE_API void OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);

	UE_API void OnAddInputPin();
	UE_API bool CanAddInputPin() const;
	UE_API void OnRemoveInputPin();
	UE_API bool CanRemoveInputPin() const;

	UE_API void OnEnableBreakpoint();
	UE_API bool CanEnableBreakpoint() const;
	UE_API void OnToggleBreakpoint();
	UE_API bool CanToggleBreakpoint() const;
	UE_API void OnDisableBreakpoint();
	UE_API bool CanDisableBreakpoint() const;
	UE_API void OnAddBreakpoint();
	UE_API bool CanAddBreakpoint() const;
	UE_API void OnRemoveBreakpoint();
	UE_API bool CanRemoveBreakpoint() const;

	UE_API void SearchTree();
	UE_API bool CanSearchTree() const;

	UE_API void JumpToNode(const UEdGraphNode* Node);

	UE_API bool IsPropertyEditable() const;
	UE_API void OnPackageSaved(const FString& PackageFileName, UPackage* Package, FObjectPostSaveContext ObjectSaveContext);
	UE_API void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);
	UE_API virtual void OnClassListUpdated() override;

	UE_API void UpdateToolbar();
	UE_API bool IsDebuggerReady() const;

	/** Get whether the debugger is currently running and the PIE session is paused */
	UE_API bool IsDebuggerPaused() const;

	/** Get whether we can edit the tree/blackboard with the debugger active */
	UE_API bool CanEditWithDebuggerActive() const;

	UE_API TSharedRef<class SWidget> OnGetDebuggerActorsMenu();
	UE_API void OnDebuggerActorSelected(TWeakObjectPtr<UBehaviorTreeComponent> InstanceToDebug);
	UE_API FText GetDebuggerActorDesc() const;
	UE_API FGraphAppearanceInfo GetGraphAppearance() const;
	UE_API bool InEditingMode(bool bGraphIsEditable) const;

	UE_API void DebuggerSwitchAsset(UBehaviorTree* NewAsset);
	UE_API void DebuggerUpdateGraph();

	UE_API EVisibility GetDebuggerDetailsVisibility() const;
	UE_API EVisibility GetRangeLowerVisibility() const;
	UE_API EVisibility GetRangeSelfVisibility() const;
	UE_API EVisibility GetInjectedNodeVisibility() const;
	UE_API EVisibility GetRootLevelNodeVisibility() const;

	UE_API TWeakPtr<SGraphEditor> GetFocusedGraphPtr() const;

	/** Check whether the behavior tree mode can be accessed (i.e whether we have a valid tree to edit) */
	UE_API bool CanAccessBehaviorTreeMode() const;

	/** Check whether the blackboard mode can be accessed (i.e whether we have a valid blackboard to edit) */
	UE_API bool CanAccessBlackboardMode() const;

	/** 
	 * Get the localized text to display for the specified mode 
	 * @param	InMode	The mode to display
	 * @return the localized text representation of the mode
	 */
	static UE_API FText GetLocalizedMode(FName InMode);

	/** Access the toolbar builder for this editor */
	TSharedPtr<class FBehaviorTreeEditorToolbar> GetToolbarBuilder() { return ToolbarBuilder; }

	/** Get the behavior tree we are editing (if any) */
	UE_API UBehaviorTree* GetBehaviorTree() const;

	/** Get the blackboard we are editing (if any) */
	UE_API UBlackboardData* GetBlackboardData() const;

	/** Spawns the tab with the update graph inside */
	UE_API TSharedRef<SWidget> SpawnProperties();

	/** Spawns the search tab */
	UE_API TSharedRef<SWidget> SpawnSearch();

	/** Spawn blackboard details tab */
	UE_API TSharedRef<SWidget> SpawnBlackboardDetails();

	/** Spawn blackboard view tab */
	UE_API TSharedRef<SWidget> SpawnBlackboardView();

	/** Spawn blackboard editor tab */
	UE_API TSharedRef<SWidget> SpawnBlackboardEditor();

	// @todo This is a hack for now until we reconcile the default toolbar with application modes [duplicated from counterpart in Blueprint Editor]
	UE_API void RegisterToolbarTab(const TSharedRef<class FTabManager>& TabManager);

	/** Restores the behavior tree graph we were editing or creates a new one if none is available */
	UE_API void RestoreBehaviorTree();

	/** Save the graph state for later editing */
	UE_API void SaveEditedObjectState();

	/** Delegate handler for selection in the blackboard entry list */
	UE_API void HandleBlackboardEntrySelected(const FBlackboardEntry* BlackboardEntry, bool bIsInherited);

	/** Delegate handler used to retrieve current blackboard selection */
	UE_API int32 HandleGetSelectedBlackboardItemIndex(bool& bOutIsInherited);

	/** Delegate handler for displaying debugger values */
	UE_API FText HandleGetDebugKeyValue(const FName& InKeyName, bool bUseCurrentState) const;

	/** Delegate handler for retrieving timestamp to display */
	UE_API double HandleGetDebugTimeStamp(bool bUseCurrentState) const;

	/** Delegate handler for when the debugged blackboard changes */
	UE_API void HandleDebuggedBlackboardChanged(UBlackboardData* InObject);

	/** Delegate handler for determining whether to display the current state */
	UE_API bool HandleGetDisplayCurrentState() const;

	/** Check whether blackboard mode is current */
	UE_API bool HandleIsBlackboardModeActive() const;

	/** Get the currently selected blackboard entry */
	UE_API void GetBlackboardSelectionInfo(int32& OutSelectionIndex, bool& bOutIsInherited) const;

	/** Check to see if we can create a new task node */
	UE_API bool CanCreateNewTask() const;

	/** Check to see if we can create a new decorator node */
	UE_API bool CanCreateNewDecorator() const;

	/** Check to see if we can create a new service node */
	UE_API bool CanCreateNewService() const;

	/** Create the menu used to make a new task node */
	UE_API TSharedRef<SWidget> HandleCreateNewTaskMenu() const;

	/** Create the menu used to make a new decorator */
	UE_API TSharedRef<SWidget> HandleCreateNewDecoratorMenu() const;

	/** Create the menu used to make a new service */
	UE_API TSharedRef<SWidget> HandleCreateNewServiceMenu() const;

	/** Handler for when a node class is picked */
	UE_API void HandleNewNodeClassPicked(UClass* InClass) const;

	/** Create a new task from UBTTask_BlueprintBase */
	UE_API void CreateNewTask() const;

	/** Whether the single button to create a new Blueprint-based task is visible */
	UE_API bool IsNewTaskButtonVisible() const;

	/** Whether the combo button to create a new Blueprint-based task from all available base classes is visible */
	UE_API bool IsNewTaskComboVisible() const;

	/** Create a new decorator from UBTDecorator_BlueprintBase */
	UE_API void CreateNewDecorator() const;

	/** Whether the single button to create a new Blueprint-based decorator is visible */
	UE_API bool IsNewDecoratorButtonVisible() const;

	/** Whether the combo button to create a new Blueprint-based decorator from all available base classes is visible */
	UE_API bool IsNewDecoratorComboVisible() const;

	/** Create a new service from UBTService_BlueprintBase */
	UE_API void CreateNewService() const;

	/** Whether the single button to create a new Blueprint-based service is visible */
	UE_API bool IsNewServiceButtonVisible() const;

	/** Whether the combo button to create a new Blueprint-based service from all available base classes is visible */
	UE_API bool IsNewServiceComboVisible() const;

	/** Create a new Blackboard alongside the currently-edited behavior tree */
	UE_API void CreateNewBlackboard();

	/** Whether we can currently create a new Blackboard */
	UE_API bool CanCreateNewBlackboard() const;

protected:
	/** Called when "Save" is clicked for this asset */
	UE_API virtual void SaveAsset_Execute() override;

	UE_API void SetToolbarCreateActionsEnabled(bool bActionsEnabled);

	TSubclassOf<UBehaviorTreeGraph> GraphClass;
	FName GraphName;
	FText CornerText;
	FText TitleText;
	FText RootNodeNoteText;

private:
	/** Create widget for graph editing */
	UE_API TSharedRef<class SGraphEditor> CreateGraphEditorWidget(UEdGraph* InGraph);

	/** Creates all internal widgets for the tabs to point at */
	UE_API void CreateInternalWidgets();

	/** Add custom menu options */
	UE_API void ExtendMenu();

	/** Setup common commands */
	UE_API void BindCommonCommands();

	/** Setup commands */
	UE_API void BindDebuggerToolbarCommands();

	/** Called when the selection changes in the GraphEditor */
	UE_API virtual void OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection) override;

	/** prepare range of nodes that can be aborted by this decorator */
	UE_API void GetAbortModePreview(const class UBehaviorTreeGraphNode_CompositeDecorator* Node, struct FAbortDrawHelper& Mode0, struct FAbortDrawHelper& Mode1);

	/** prepare range of nodes that can be aborted by this decorator */
	UE_API void GetAbortModePreview(const class UBTDecorator* DecoratorOb, struct FAbortDrawHelper& Mode0, struct FAbortDrawHelper& Mode1);

	/** Refresh the debugger's display */
	UE_API void RefreshDebugger();

	/** Push new associated Blackboard data to Blackboard views */
	UE_API void RefreshBlackboardViewsAssociatedObject();

	TSharedPtr<FDocumentTracker> DocumentManager;
	TWeakPtr<FDocumentTabFactory> GraphEditorTabFactoryPtr;

	/* The Behavior Tree being edited */
	UBehaviorTree* BehaviorTree;

	/* The Blackboard Data being edited */
	UBlackboardData* BlackboardData;

	TWeakObjectPtr<class UBehaviorTreeGraphNode_CompositeDecorator> FocusedGraphOwner;

	/** Property View */
	TSharedPtr<class IDetailsView> DetailsView;

	TSharedPtr<class FBehaviorTreeDebugger> Debugger;

	/** Find results log as well as the search filter */
	TSharedPtr<class SFindInBT> FindResults;

	uint32 bShowDecoratorRangeLower : 1;
	uint32 bShowDecoratorRangeSelf : 1;
	uint32 bForceDisablePropertyEdit : 1;
	uint32 bSelectedNodeIsInjected : 1;
	uint32 bSelectedNodeIsRootLevel : 1;

	uint32 bHasMultipleTaskBP : 1;
	uint32 bHasMultipleDecoratorBP : 1;
	uint32 bHasMultipleServiceBP : 1;

	TSharedPtr<class FBehaviorTreeEditorToolbar> ToolbarBuilder;

	/** The details view we use to display the blackboard */
	TSharedPtr<IDetailsView> BlackboardDetailsView;

	/** The blackboard view widget */
	TSharedPtr<class SBehaviorTreeBlackboardView> BlackboardView;

	/** The blackboard editor widget */
	TSharedPtr<class SBehaviorTreeBlackboardEditor> BlackboardEditor;

	/** The current blackboard selection index, stored here so it can be accessed by our details customization */
	int32 CurrentBlackboardEntryIndex;

	/** Whether the current selection is inherited, stored here so it can be accessed by our details customization */
	bool bIsCurrentBlackboardEntryInherited;

	/** Handle to the registered OnPackageSave delegate */
	FDelegateHandle OnPackageSavedDelegateHandle;

public:
	/** Modes in mode switcher */
	static UE_API const FName BehaviorTreeMode;
	static UE_API const FName BlackboardMode;

	static UE_API FText BehaviorTreeModeText;
	static UE_API FText BlackboardModeText;
};

#undef UE_API
