// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/AssetEditorToolkit.h"

class IBehaviorTreeEditor;
struct FGraphNodeClassHelper;

DECLARE_LOG_CATEGORY_EXTERN(LogBehaviorTreeEditor, Log, All);

/** Behavior Tree Editor module */
class FBehaviorTreeEditorModule : public IModuleInterface, public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	// IModuleInterface
	BEHAVIORTREEEDITOR_API virtual void StartupModule() override;
	BEHAVIORTREEEDITOR_API virtual void ShutdownModule() override;

	/** Creates an instance of Behavior Tree editor.  Only virtual so that it can be called across the DLL boundary. */
	BEHAVIORTREEEDITOR_API virtual TSharedRef<IBehaviorTreeEditor> CreateBehaviorTreeEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UObject* Object);

	/** Gets the extensibility managers for outside entities to extend static mesh editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

	TSharedPtr<FGraphNodeClassHelper> GetClassCache() const { return ClassCache; }

	/** Behavior Tree app identifier string */
	static BEHAVIORTREEEDITOR_API const FName BehaviorTreeEditorAppIdentifier;

private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

	TSharedPtr<FGraphNodeClassHelper> ClassCache;
};

