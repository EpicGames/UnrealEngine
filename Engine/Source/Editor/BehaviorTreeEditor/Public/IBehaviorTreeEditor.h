// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

class UEdGraphNode;
struct Rect;

/** BT Editor public interface */
class IBehaviorTreeEditor : public FWorkflowCentricApplication
{

public:
	virtual void InitializeDebuggerState(class FBehaviorTreeDebugger* ParentDebugger) const = 0;
	virtual UEdGraphNode* FindInjectedNode(int32 Index) const = 0;
	virtual void DoubleClickNode(class UEdGraphNode* Node) = 0;
	virtual void FocusAttentionOnNode(UEdGraphNode* Node) = 0;
};


